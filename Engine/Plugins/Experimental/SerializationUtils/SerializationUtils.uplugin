{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Serialization Utils",
	"Description": "Utilities for serialization (xml, json, etc) with extended functionality.",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "JsonSerialization",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "XmlSerialization",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}