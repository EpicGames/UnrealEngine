// Copyright Epic Games, Inc. All Rights Reserved.

#include "Formatters/JsonArchiveInputFormatterEx.h"
#include "Dom/JsonObject.h"
#include "Misc/Base64.h"
#include "Misc/SecureHash.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"

#if WITH_TEXT_ARCHIVE_SUPPORT

namespace UE::JsonArchiveInputFormatter::Private
{
	void SerializeObject(FStructuredArchiveRecord& InRootRecord
	, UObject* InObject
	, const TFunction<bool(FProperty*)>& ShouldSkipProperty)
	{
		if (!InObject)
		{
			return;
		}
	
		FStructuredArchiveRecord ObjectRecord = InRootRecord.EnterRecord(*InObject->GetName());
		FArchive& UnderlyingArchive = InRootRecord.GetUnderlyingArchive();
	
		for (FProperty* Property = InObject->GetClass()->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			if (!Property || Property->HasAnyPropertyFlags(CPF_Transient) || (ShouldSkipProperty && ShouldSkipProperty(Property)))
			{
				continue;
			}

			FStructuredArchiveSlot PropertyField = InRootRecord.EnterField(*Property->GetName());

			for (int32 Index = 0; Index < Property->ArrayDim; ++Index)
			{
				UObject* const Archetype = InObject->GetArchetype();
			
				void* const Data = Property->ContainerPtrToValuePtr<void>(InObject, Index);
				const void* const Defaults = Property->ContainerPtrToValuePtrForDefaults<void>(Archetype->GetClass()
					, Archetype
					, Index);
			
				{
					FSerializedPropertyScope SerializedProperty(UnderlyingArchive, Property);
					Property->SerializeItem(PropertyField, Data, Defaults);
				}
			}
		}
	}
};

FJsonArchiveInputFormatterEx::FJsonArchiveInputFormatterEx(FArchive& InInner, UObject* InRootObject, TFunction<UObject* (const FPackageIndex)> InResolveObject) 
	: Inner(InInner)
	, ResolveObject(InResolveObject)
	, RootObject(InRootObject)
{
	Inner.SetIsTextFormat(true);
	Inner.ArAllowLazyLoading = false;

	TSharedPtr< FJsonObject > RootJsonObject;
	TSharedRef< TJsonReader<UTF8CHAR> > Reader = TJsonReaderFactory<UTF8CHAR>::Create(&InInner);
	ensure(FJsonSerializer::Deserialize(Reader, RootJsonObject, FJsonSerializer::EFlags::None));
	ValueStack.Add(MakeShared<FJsonValueObject>(RootJsonObject));

	ValueStack.Reserve(64);
	ArrayValuesRemainingStack.Reserve(64);
}

FJsonArchiveInputFormatterEx::~FJsonArchiveInputFormatterEx()
{
}

FArchive& FJsonArchiveInputFormatterEx::GetUnderlyingArchive()
{
	return Inner;
}

FStructuredArchiveFormatter* FJsonArchiveInputFormatterEx::CreateSubtreeReader()
{
	FJsonArchiveInputFormatterEx* Cloned = new FJsonArchiveInputFormatterEx(*this);
	Cloned->ObjectStack.Empty();
	Cloned->ValueStack.Empty();
	Cloned->MapIteratorStack.Empty();
	Cloned->ArrayValuesRemainingStack.Empty();
	Cloned->ValueStack.Push(ValueStack.Top());

	return Cloned;
}

bool FJsonArchiveInputFormatterEx::HasDocumentTree() const
{
	return true;
}

void FJsonArchiveInputFormatterEx::EnterRecord()
{
	TSharedPtr<FJsonValue>& Value = ValueStack.Top();
	ObjectStack.Emplace(Value->AsObject(), ValueStack.Num());
}

void FJsonArchiveInputFormatterEx::LeaveRecord()
{
	// In the original, this check is in place for debugging purposes and to check that all values are consumed
	// check(ValueStack.Num() == ObjectStack.Top().ValueCountOnCreation);
	ObjectStack.Pop();
}

void FJsonArchiveInputFormatterEx::EnterField(FArchiveFieldName Name)
{
	TSharedPtr<FJsonObject>& NewObject = ObjectStack.Top().JsonObject;
	TSharedPtr<FJsonValue> Field = NewObject->TryGetField(EscapeFieldName(Name.Name));
	check(Field.IsValid());
	ValueStack.Add(Field);
}

void FJsonArchiveInputFormatterEx::LeaveField()
{
	ValueStack.Pop();
}

bool FJsonArchiveInputFormatterEx::TryEnterField(FArchiveFieldName Name, bool bEnterWhenSaving)
{
	TSharedPtr<FJsonObject>& NewObject = ObjectStack.Top().JsonObject;
	TSharedPtr<FJsonValue> Field = NewObject->TryGetField(EscapeFieldName(Name.Name));
	if (Field.IsValid())
	{
		ValueStack.Add(Field);
		return true;
	}
	return false;
}

void FJsonArchiveInputFormatterEx::EnterArray(int32& NumElements)
{
	TSharedPtr<FJsonValue>& Value = ValueStack.Top();

	const TArray<TSharedPtr<FJsonValue>>& ArrayValue = Value->AsArray();
	for(int Idx = ArrayValue.Num() - 1; Idx >= 0; Idx--)
	{
		ValueStack.Add(ArrayValue[Idx]);
	}

	NumElements = ArrayValue.Num();
	ArrayValuesRemainingStack.Add(NumElements);
}

void FJsonArchiveInputFormatterEx::LeaveArray()
{
	check(ArrayValuesRemainingStack.Num() > 0);
	int32 Remaining = ArrayValuesRemainingStack.Top();
	ArrayValuesRemainingStack.Pop();
	check(Remaining >= 0);
	check(Remaining <= ValueStack.Num());
	while (Remaining-- > 0)
	{
		ValueStack.Pop();
	}
}

void FJsonArchiveInputFormatterEx::EnterArrayElement()
{
	check(ArrayValuesRemainingStack.Num() > 0);
	check(ArrayValuesRemainingStack.Top() > 0);
}

void FJsonArchiveInputFormatterEx::LeaveArrayElement()
{
	ValueStack.Pop();
	ArrayValuesRemainingStack.Top()--;
}

void FJsonArchiveInputFormatterEx::EnterStream()
{
	int32 NumElements = 0;
	EnterArray(NumElements);
}

void FJsonArchiveInputFormatterEx::LeaveStream()
{
	LeaveArray();
}

void FJsonArchiveInputFormatterEx::EnterStreamElement()
{
}

void FJsonArchiveInputFormatterEx::LeaveStreamElement()
{
	LeaveArrayElement();
}

void FJsonArchiveInputFormatterEx::EnterMap(int32& NumElements)
{
	TSharedPtr<FJsonValue>& Value = ValueStack.Top();
	TSharedPtr<FJsonObject> Object = Value->AsObject();
	MapIteratorStack.Add(Object->Values.CreateIterator());
	NumElements = Object->Values.Num();
}

void FJsonArchiveInputFormatterEx::LeaveMap()
{
	MapIteratorStack.Pop();
}

void FJsonArchiveInputFormatterEx::EnterMapElement(FString& OutName)
{
	OutName = UnescapeFieldName(*MapIteratorStack.Top()->Key);
	ValueStack.Add(MapIteratorStack.Top()->Value);
}

void FJsonArchiveInputFormatterEx::LeaveMapElement()
{
	ValueStack.Pop();
	++MapIteratorStack.Top();
}

void FJsonArchiveInputFormatterEx::EnterAttributedValue()
{
	TSharedPtr<FJsonValue>& Value = ValueStack.Top();
	const TSharedPtr<FJsonObject>* ObjectPtr = nullptr;
	if (Value->TryGetObject(ObjectPtr))
	{
		FJsonObject& ObjectRef = **ObjectPtr;

		TSharedPtr<FJsonValue> Field = ObjectRef.TryGetField(EscapeFieldName(TEXT("_Value")));
		if (Field.IsValid())
		{
			ObjectStack.Emplace(*ObjectPtr, ValueStack.Num());
			return;
		}
	}

	ObjectStack.Emplace(nullptr, ValueStack.Num());
}

void FJsonArchiveInputFormatterEx::EnterAttribute(FArchiveFieldName AttributeName)
{
	TSharedPtr<FJsonObject>& NewObject = ObjectStack.Top().JsonObject;
	TSharedPtr<FJsonValue> Field = NewObject->TryGetField(EscapeFieldName(*FString::Printf(TEXT("_%s"), AttributeName.Name)));
	check(Field.IsValid());
	ValueStack.Add(Field);
}

void FJsonArchiveInputFormatterEx::EnterAttributedValueValue()
{
	if (TSharedPtr<FJsonObject>& Object = ObjectStack.Top().JsonObject)
	{
		TSharedPtr<FJsonValue> Field = Object->TryGetField(EscapeFieldName(TEXT("_Value")));
		checkSlow(Field);
		ValueStack.Add(Field);
	}
	else
	{
		ValueStack.Add(CopyTemp(ValueStack.Top()));
	}
}

bool FJsonArchiveInputFormatterEx::TryEnterAttributedValueValue()
{
	TSharedPtr<FJsonValue>& Value = ValueStack.Top();
	const TSharedPtr<FJsonObject>* ObjectPtr = nullptr;
	if (Value->TryGetObject(ObjectPtr))
	{
		FJsonObject& ObjectRef = **ObjectPtr;

		TSharedPtr<FJsonValue> Field = ObjectRef.TryGetField(EscapeFieldName(TEXT("_Value")));
		if (Field.IsValid())
		{
			ObjectStack.Emplace(*ObjectPtr, ValueStack.Num());
			ValueStack.Add(Field);
			return true;
		}
	}

	return false;
}

void FJsonArchiveInputFormatterEx::LeaveAttribute()
{
	ValueStack.Pop();
}

void FJsonArchiveInputFormatterEx::LeaveAttributedValue()
{
	// check(ValueStack.Num() == ObjectStack.Top().ValueCountOnCreation);
	ObjectStack.Pop();
}

bool FJsonArchiveInputFormatterEx::TryEnterAttribute(FArchiveFieldName AttributeName, bool bEnterWhenSaving)
{
	if (TSharedPtr<FJsonObject>& Object = ObjectStack.Top().JsonObject)
	{
		if (TSharedPtr<FJsonValue> Field = Object->TryGetField(EscapeFieldName(TEXT("_Value"))))
		{
			TSharedPtr<FJsonValue> Attribute = Object->TryGetField(EscapeFieldName(*FString::Printf(TEXT("_%s"), AttributeName.Name)));
			if (Attribute.IsValid())
			{
				ValueStack.Add(Attribute);
				return true;
			}
		}
	}
	return false;
}

void FJsonArchiveInputFormatterEx::Serialize(uint8& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

void FJsonArchiveInputFormatterEx::Serialize(uint16& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

void FJsonArchiveInputFormatterEx::Serialize(uint32& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

void FJsonArchiveInputFormatterEx::Serialize(uint64& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

void FJsonArchiveInputFormatterEx::Serialize(int8& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

void FJsonArchiveInputFormatterEx::Serialize(int16& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

void FJsonArchiveInputFormatterEx::Serialize(int32& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

void FJsonArchiveInputFormatterEx::Serialize(int64& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

void FJsonArchiveInputFormatterEx::Serialize(float& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

void FJsonArchiveInputFormatterEx::Serialize(double& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

void FJsonArchiveInputFormatterEx::Serialize(bool& Value)
{
	Value = ValueStack.Top()->AsBool();
}

void FJsonArchiveInputFormatterEx::Serialize(UTF32CHAR& Value)
{
	verify(ValueStack.Top()->TryGetNumber(*(uint32*)&Value));
}

void FJsonArchiveInputFormatterEx::Serialize(FString& Value)
{
	// If the string we serialized was empty, this value will be a null object rather than a string, so we have to
#if DO_CHECK
	bool bSuccess =
#endif
		ValueStack.Top()->TryGetString(Value);
	check(bSuccess || ValueStack.Top()->IsNull());
	Value.RemoveFromStart(TEXT("String:"));
}

void FJsonArchiveInputFormatterEx::Serialize(FName& Value)
{
	FString StringValue = ValueStack.Top()->AsString();
	Value = FName(*StringValue);
}

void FJsonArchiveInputFormatterEx::Serialize(UObject*& Value)
{
	if (IsNestedObject())
	{
		Value = LoadNestedObject();
		return;
	}
	
	FString StringValue;
	if (ValueStack.Top()->TryGetString(StringValue))
	{
		FPackageIndex ObjectIndex;
		LexFromString(ObjectIndex, *StringValue);
		Value = ResolveObject(ObjectIndex);
	}
	else
	{
		Value = nullptr;
	}
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
void FJsonArchiveInputFormatterEx::Serialize(Verse::VCell*& Value)
{
	Value = nullptr;
}
#endif

void FJsonArchiveInputFormatterEx::Serialize(FText& Value)
{
	// FStructuredArchive ChildArchive(*this);
	// FText::SerializeText(ChildArchive.Open(), Value);
	// ChildArchive.Close();
}

void FJsonArchiveInputFormatterEx::Serialize(FWeakObjectPtr& Value)
{
	UObject* Object = nullptr;
	Serialize(Object);
	Value = Object;
}

void FJsonArchiveInputFormatterEx::Serialize(FSoftObjectPtr& Value)
{
	// Special case for nested objects.
	if (IsNestedObject())
	{
		Value = LoadNestedObject();
	}
	else
	{
		FSoftObjectPath Path;
		Serialize(Path);
		Value = Path;
	}
}

void FJsonArchiveInputFormatterEx::Serialize(FSoftObjectPath& Value)
{
	FString StringValue;
	const auto& Prefix = TEXT("Object:");
	if (ValueStack.Top()->TryGetString(StringValue) && StringValue.StartsWith(Prefix))
	{
		Value.SetPath(*StringValue + UE_ARRAY_COUNT(Prefix) - 1);
	}
	else
	{
		Value.Reset();
	}
}

void FJsonArchiveInputFormatterEx::Serialize(FLazyObjectPtr& Value)
{
	FString StringValue;
	const auto& Prefix = TEXT("Lazy:");
	if (ValueStack.Top()->TryGetString(StringValue) && StringValue.StartsWith(Prefix))
	{
		FUniqueObjectGuid Guid;
		Guid.FromString(*StringValue + UE_ARRAY_COUNT(Prefix) - 1);
		Value = Guid;
	}
	else
	{
		Value.Reset();
	}
}

void FJsonArchiveInputFormatterEx::Serialize(FObjectPtr& Value)
{
	UObject* Object;
	Serialize(Object);
	Value = Object;
}

void FJsonArchiveInputFormatterEx::Serialize(TArray<uint8>& Data)
{
	const FJsonValue& Value = *ValueStack.Top();
	if(Value.Type == EJson::String)
	{
		// Single line string
		FString RawData = ValueStack.Top()->AsString();
		ensure(RawData.RemoveFromStart(TEXT("Base64:")));
		FBase64::Decode(RawData, Data);
	}
	else if(Value.Type == EJson::Object)
	{
		// Multi-line string
		FJsonObject& Object = *Value.AsObject();

		// Read the digest
		TSharedPtr<FJsonValue> DigestField = Object.TryGetField(TEXT("Digest"));
		checkf(DigestField.IsValid() && DigestField->Type == EJson::String, TEXT("Missing or invalid 'Digest' field for raw data"));
		FString Digest = DigestField->AsString();

		// Read the base64 array
		TSharedPtr<FJsonValue> Base64Field = Object.TryGetField(TEXT("Base64"));
		checkf(Base64Field.IsValid() && Base64Field->Type == EJson::Array, TEXT("Missing or invalid 'Base64' field for raw data"));
		const TArray<TSharedPtr<FJsonValue>>& Base64Array = Base64Field->AsArray();

		// Parse the digest
		uint8 ExpectedDigest[FSHA1::DigestSize];
		verify(FString::ToHexBlob(DigestField->AsString(), ExpectedDigest, sizeof(ExpectedDigest)));

		// Get the size of the encoded data
		uint32 DecodedSize = 0;
		for(const TSharedPtr<FJsonValue>& Base64Line : Base64Array)
		{
			DecodedSize += FBase64::GetDecodedDataSize(Base64Line->AsString());
		}

		// Read the encoded data
		Data.SetNum(DecodedSize);

		// Read each line
		uint32 DecodedPos = 0;
		for(const TSharedPtr<FJsonValue>& Base64Line : Base64Array)
		{
			FString Base64String = Base64Line->AsString();
			verify(FBase64::Decode(*Base64String, Base64String.Len(), Data.GetData() + DecodedPos));
			DecodedPos += FBase64::GetDecodedDataSize(Base64String);
		}

		// Make sure the digest matches
		uint8 ActualDigest[FSHA1::DigestSize];
		FSHA1::HashBuffer(Data.GetData(), Data.Num(), ActualDigest);
		checkf(FMemory::Memcmp(ExpectedDigest, ActualDigest, FSHA1::DigestSize) == 0, TEXT("Digest does not match for raw data. Check that this file was merged correctly."));
	}
	else
	{
		checkf(false, TEXT("Invalid value type for raw data"));
	}
}

void FJsonArchiveInputFormatterEx::Serialize(void* Data, uint64 DataSize)
{
	TArray<uint8> Buffer;
	Serialize(Buffer);
	check(Buffer.Num() == DataSize);
	memcpy(Data, Buffer.GetData(), DataSize);
}

bool FJsonArchiveInputFormatterEx::IsNestedObject()
{
	return TryEnterField(TEXT("Class"), false);
}

UObject* FJsonArchiveInputFormatterEx::LoadNestedObject()
{
	FString ClassPathString;
	Serialize(ClassPathString);
	const FSoftObjectPath ClassPath(ClassPathString);
	const TSoftObjectPtr<UClass> ClassPtr(ClassPath);
	if (const UClass* const ObjectClass = ClassPtr.LoadSynchronous())
	{
		uint32 flags;
		EnterField(TEXT("Flags"));
		Serialize(flags);
		const EObjectFlags ObjectFlags = static_cast<EObjectFlags>(flags);

		FString OriginalObjectName;
		EnterField(TEXT("Name"));
		Serialize(OriginalObjectName);

		// Name clashing prevention: in order to prevent name clashing, we let the new object be named with
		// a unique name and rename the root field tag.
		UObject* const NestedObject = NewObject<UObject>(RootObject, ObjectClass, *OriginalObjectName, ObjectFlags);

		if (!NestedObject)
		{
			return nullptr;
		}

		{
			EnterField(*OriginalObjectName);
			FStructuredArchive ChildArchive(*this);
			FStructuredArchiveRecord ObjectRecord = ChildArchive.Open().EnterRecord();
			
			for (FProperty* Property = NestedObject->GetClass()->PropertyLink; Property; Property = Property->PropertyLinkNext)
			{
				if (!Property || Property->HasAnyPropertyFlags(CPF_Transient))
				{
					continue;
				}

				FStructuredArchiveSlot PropertyField = ObjectRecord.EnterField(*Property->GetName());
		
				for (int32 Index = 0; Index < Property->ArrayDim; ++Index)
				{
					UObject* const Archetype = NestedObject->GetArchetype();
			
					void* const Data = Property->ContainerPtrToValuePtr<void>(NestedObject, Index);
					const void* const Defaults = Property->ContainerPtrToValuePtrForDefaults<void>(Archetype->GetClass()
						, Archetype
						, Index);
			
					//if (!Property->Identical(Target, Default, UnderlyingArchive.GetPortFlags()))
					{
						FSerializedPropertyScope SerializedProperty(GetUnderlyingArchive(), Property);
						Property->SerializeItem(PropertyField, Data, Defaults);
					}
				}
			}
			// FStructuredArchive ChildArchive(*this);
			// FStructuredArchiveRecord ObjectRecord = ChildArchive.Open().EnterRecord();
			// UE::JsonArchiveInputFormatter::Private::SerializeObject(ObjectRecord, NestedObject, TFunction<bool(FProperty*)>());
			// LeaveRecord();
			ChildArchive.Close();
		}

		NestedObject->PostLoad();

		return NestedObject;
	}
	
	return nullptr;
}

FString FJsonArchiveInputFormatterEx::EscapeFieldName(const TCHAR* Name)
{
	if(FCString::Stricmp(Name, TEXT("Base64")) == 0 || FCString::Stricmp(Name, TEXT("Digest")) == 0 || Name[0] == '_')
	{
		return FString::Printf(TEXT("_%s"), Name);
	}
	else
	{
		return Name;
	}
}

FString FJsonArchiveInputFormatterEx::UnescapeFieldName(const TCHAR* Name)
{
	if(Name[0] == '_')
	{
		return Name + 1;
	}
	else
	{
		return Name;
	}
}

EArchiveValueType FJsonArchiveInputFormatterEx::GetValueType(const FJsonValue& Value)
{
	switch (Value.Type)
	{
	case EJson::String:
		if (Value.AsString().StartsWith(TEXT("Object:")))
		{
			return EArchiveValueType::Object;
		}
		else if (Value.AsString().StartsWith(TEXT("Base64:")))
		{
			return EArchiveValueType::RawData;
		}
		else
		{
			return EArchiveValueType::String;
		}
	case EJson::Number:
		{
			double Number = Value.AsNumber();
			int64 NumberInt64 = (int64)Number;
			if((double)NumberInt64 == Number)
			{
				if((int16)NumberInt64 == NumberInt64)
				{
					if((int8)NumberInt64 == NumberInt64)
					{
						return EArchiveValueType::Int8;
					}
					else
					{
						return EArchiveValueType::Int16;
					}
				}
				else
				{
					if((int32)NumberInt64 == NumberInt64)
					{
						return EArchiveValueType::Int32;
					}
					else
					{
						return EArchiveValueType::Int64;
					}
				}
			}
			else
			{
				if((double)(float)Number == Number)
				{
					return EArchiveValueType::Float;
				}
				else
				{
					return EArchiveValueType::Double;
				}
			}
		}
	case EJson::Boolean:
		return EArchiveValueType::Bool;
	case EJson::Array:
		return EArchiveValueType::Array;
	case EJson::Object:
		if(Value.AsObject()->HasField(TEXT("Base64")))
		{
			return EArchiveValueType::RawData;
		}
		else
		{
			return EArchiveValueType::Record;
		}
	case EJson::Null:
		return EArchiveValueType::Object;
	default:
		checkf(false, TEXT("Unhandled value type in JSON archive (%d)"), (int)Value.Type);
		return EArchiveValueType::None;
	}
}

#endif
