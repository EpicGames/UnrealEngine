// Copyright Epic Games, Inc. All Rights Reserved.

#include "Formatters/JsonArchiveOutputFormatterEx.h"
#include "Misc/Base64.h"
#include "Misc/SecureHash.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"

#if WITH_TEXT_ARCHIVE_SUPPORT

namespace UE::JsonArchiveOutputFormatter::Private
{
	void SerializeObject(FStructuredArchiveRecord& InRootRecord
		, UObject* InObject
		, const TFunction<bool(FProperty*)>& ShouldSkipProperty = TFunction<bool(FProperty*)>())
	{
		if (!InObject)
		{
			return;
		}
	
		FStructuredArchiveRecord ObjectRecord = InRootRecord.EnterRecord(*InObject->GetName());
		FArchive& UnderlyingArchive = InRootRecord.GetUnderlyingArchive();
	
		for (FProperty* Property = InObject->GetClass()->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			if (!Property || Property->HasAnyPropertyFlags(CPF_Transient) || (ShouldSkipProperty && ShouldSkipProperty(Property)))
			{
				continue;
			}

			FStructuredArchiveSlot PropertyField = ObjectRecord.EnterField(*Property->GetName());
		
			for (int32 Index = 0; Index < Property->ArrayDim; ++Index)
			{
				UObject* const Archetype = InObject->GetArchetype();
			
				void* const Data = Property->ContainerPtrToValuePtr<void>(InObject, Index);
				const void* const Defaults = Property->ContainerPtrToValuePtrForDefaults<void>(Archetype->GetClass()
					, Archetype
					, Index);
			
				//if (!Property->Identical(Target, Default, UnderlyingArchive.GetPortFlags()))
				{
					FSerializedPropertyScope SerializedProperty(UnderlyingArchive, Property);
					Property->SerializeItem(PropertyField, Data, Defaults);
				}
			}
		}
	}
};

FJsonArchiveOutputFormatterEx::FJsonArchiveOutputFormatterEx(FArchive& InInner) 
	: Inner(InInner)
	, Newline(LINE_TERMINATOR_ANSI, UE_ARRAY_COUNT(LINE_TERMINATOR_ANSI) - 1)
{
	Inner.SetIsTextFormat(true);
}

FJsonArchiveOutputFormatterEx::~FJsonArchiveOutputFormatterEx()
{
}

FArchive& FJsonArchiveOutputFormatterEx::GetUnderlyingArchive()
{
	return Inner;
}

bool FJsonArchiveOutputFormatterEx::HasDocumentTree() const
{
	return true;
}

void FJsonArchiveOutputFormatterEx::EnterRecord()
{
	WriteOptionalComma();
	WriteOptionalNewline();
	Write("{");
	Newline.Add('\t');
	bNeedsNewline = true;
	TextStartPosStack.Push(Inner.Tell());
}

void FJsonArchiveOutputFormatterEx::LeaveRecord()
{
	Newline.Pop(EAllowShrinking::No);
	if (TextStartPosStack.Pop() == Inner.Tell())
	{
		bNeedsNewline = false;
	}
	WriteOptionalNewline();
	Write("}");
	bNeedsComma = true;
	bNeedsNewline = true;
}

void FJsonArchiveOutputFormatterEx::EnterField(FArchiveFieldName Name)
{
	WriteOptionalComma();
	WriteOptionalNewline();
	WriteFieldName(Name.Name);
}

void FJsonArchiveOutputFormatterEx::LeaveField()
{
	bNeedsComma = true;
	bNeedsNewline = true;
}

bool FJsonArchiveOutputFormatterEx::TryEnterField(FArchiveFieldName Name, bool bEnterWhenSaving)
{
	if (bEnterWhenSaving)
	{
		EnterField(Name);
	}
	return bEnterWhenSaving;
}

void FJsonArchiveOutputFormatterEx::EnterArray(int32& NumElements)
{
	EnterStream();
}

void FJsonArchiveOutputFormatterEx::LeaveArray()
{
	LeaveStream();
}

void FJsonArchiveOutputFormatterEx::EnterArrayElement()
{
	EnterStreamElement();
}

void FJsonArchiveOutputFormatterEx::LeaveArrayElement()
{
	LeaveStreamElement();
}

void FJsonArchiveOutputFormatterEx::EnterStream()
{
	WriteOptionalComma();
	WriteOptionalNewline();
	Write("[");
	Newline.Add('\t');
	bNeedsNewline = true;
	TextStartPosStack.Push(Inner.Tell());
}

void FJsonArchiveOutputFormatterEx::LeaveStream()
{
	Newline.Pop(EAllowShrinking::No);
	if (TextStartPosStack.Pop() == Inner.Tell())
	{
		bNeedsNewline = false;
	}
	WriteOptionalNewline();
	Write("]");
	bNeedsComma = true;
	bNeedsNewline = true;
}

void FJsonArchiveOutputFormatterEx::EnterStreamElement()
{
	WriteOptionalComma();
	WriteOptionalNewline();
}

void FJsonArchiveOutputFormatterEx::LeaveStreamElement()
{
	bNeedsComma = true;
	bNeedsNewline = true;
}

void FJsonArchiveOutputFormatterEx::EnterMap(int32& NumElements)
{
	EnterRecord();
}

void FJsonArchiveOutputFormatterEx::LeaveMap()
{
	LeaveRecord();
}

void FJsonArchiveOutputFormatterEx::EnterMapElement(FString& Name)
{
	EnterField(FArchiveFieldName(*Name));
}

void FJsonArchiveOutputFormatterEx::LeaveMapElement()
{
	LeaveField();
}

void FJsonArchiveOutputFormatterEx::EnterAttributedValue()
{
	NumAttributesStack.Push(0);
}

void FJsonArchiveOutputFormatterEx::EnterAttribute(FArchiveFieldName AttributeName)
{
	WriteOptionalComma();
	WriteOptionalNewline();
	WriteOptionalAttributedBlockOpening();
	WriteOptionalComma();
	WriteOptionalNewline();
	checkf(FCString::Strcmp(AttributeName.Name, TEXT("Value")) != 0, TEXT("Attributes called 'Value' are reserved by the implementation"));
	WriteFieldName(*FString::Printf(TEXT("_%s"), AttributeName.Name));
	++NumAttributesStack.Top();
}

void FJsonArchiveOutputFormatterEx::LeaveAttribute()
{
	bNeedsComma = true;
	bNeedsNewline = true;
}

void FJsonArchiveOutputFormatterEx::LeaveAttributedValue()
{
	WriteOptionalAttributedBlockClosing();
	NumAttributesStack.Pop();
	bNeedsComma = true;
	bNeedsNewline = true;
}

void FJsonArchiveOutputFormatterEx::EnterAttributedValueValue()
{
	WriteOptionalComma();
	WriteOptionalNewline();
	WriteOptionalAttributedBlockValue();
}

bool FJsonArchiveOutputFormatterEx::TryEnterAttributedValueValue()
{
	return false;
}

bool FJsonArchiveOutputFormatterEx::TryEnterAttribute(FArchiveFieldName AttributeName, bool bEnterWhenSaving)
{
	if (bEnterWhenSaving)
	{
		EnterAttribute(AttributeName);
	}
	return bEnterWhenSaving;
}

void FJsonArchiveOutputFormatterEx::Serialize(uint8& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatterEx::Serialize(uint16& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatterEx::Serialize(uint32& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatterEx::Serialize(uint64& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatterEx::Serialize(int8& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatterEx::Serialize(int16& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatterEx::Serialize(int32& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatterEx::Serialize(int64& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatterEx::Serialize(float& Value)
{
	if((float)(int)Value == Value)
	{
		WriteValue(LexToString((int)Value));
	}
	else
	{
		FString String = FString::Printf(TEXT("%.17g"), Value);
#if DO_GUARD_SLOW
		float RoundTripped;
		LexFromString(RoundTripped, *String);
		check(RoundTripped == Value);
#endif
		WriteValue(String);
	}
}

void FJsonArchiveOutputFormatterEx::Serialize(double& Value)
{
	if((double)(int)Value == Value)
	{
		WriteValue(LexToString((int)Value));
	}
	else
	{
		FString String = FString::Printf(TEXT("%.17g"), Value);
#if DO_GUARD_SLOW
		double RoundTripped;
		LexFromString(RoundTripped, *String);
		check(RoundTripped == Value);
#endif
		WriteValue(String);
	}
}

void FJsonArchiveOutputFormatterEx::Serialize(bool& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatterEx::Serialize(UTF32CHAR& Value)
{
	WriteValue(LexToString(*(uint32*)&Value));
}

void FJsonArchiveOutputFormatterEx::Serialize(FString& Value)
{
	// Insert a "String:" prefix to prevent incorrect interpretation as another explicit type
	if (Value.StartsWith(TEXT("Object:")) || Value.StartsWith(TEXT("String:")) || Value.StartsWith(TEXT("Base64:")))
	{
		SerializeStringInternal(FString::Printf(TEXT("String:%s"), *Value));
	}
	else
	{
		SerializeStringInternal(Value);
	}
}

void FJsonArchiveOutputFormatterEx::Serialize(FName& Value)
{
	SerializeStringInternal(*Value.ToString());
}

void FJsonArchiveOutputFormatterEx::Serialize(UObject*& Value)
{
	if (!bSerializeObjectsInPlace)
	{
		if (Value != nullptr && IsObjectAllowed(Value))
		{
			FPackageIndex ObjectIndex = ObjectIndicesMap->FindChecked(Value);
			SerializeStringInternal(LexToString(ObjectIndex));
		}
		else
		{
			WriteValue(TEXT("null"));
		}

		return;
	}

	FStructuredArchive ChildArchive(*this);
	FStructuredArchiveRecord ObjectRecord = ChildArchive.Open().EnterRecord();
	// For current record, add attribute for object type
	const TSoftObjectPtr<UClass> ObjectClass =  Value->GetClass();

	EnterField(TEXT("Class"));
	FString ObjectClassString = ObjectClass.ToSoftObjectPath().ToString();
	Serialize(ObjectClassString);
	LeaveField();

	EnterField(TEXT("Name"));
	FString Name = Value->GetName();
	Serialize(Name);
	LeaveField();

	EnterField(TEXT("Flags"));
	uint32 Flags = static_cast<uint32>(Value->GetFlags());
	Serialize(Flags);
	LeaveField();
	
	UE::JsonArchiveOutputFormatter::Private::SerializeObject(ObjectRecord, Value);
	ChildArchive.Close();
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
void FJsonArchiveOutputFormatterEx::Serialize(Verse::VCell*& Value)
{
	WriteValue(TEXT("null"));
}
#endif

void FJsonArchiveOutputFormatterEx::Serialize(FText& Value)
{
	// FStructuredArchive ChildArchive(*this);
	// FText::SerializeText(ChildArchive.Open(), Value);
	// ChildArchive.Close();
}

void FJsonArchiveOutputFormatterEx::Serialize(FWeakObjectPtr& Value)
{
	UObject* Ptr = Value.IsValid() ? Value.Get() : nullptr;
	Serialize(Ptr);
}

void FJsonArchiveOutputFormatterEx::Serialize(FSoftObjectPtr& Value)
{
	FSoftObjectPath Path = Value.ToSoftObjectPath();
	Serialize(Path);
}

void FJsonArchiveOutputFormatterEx::Serialize(FSoftObjectPath& Value)
{
	if (Value.IsValid())
	{
		SerializeStringInternal(FString::Printf(TEXT("Object:%s"), *Value.ToString()));
	}
	else
	{
		WriteValue(TEXT("null"));
	}
}

void FJsonArchiveOutputFormatterEx::Serialize(FLazyObjectPtr& Value)
{
	if (Value.IsValid() && IsObjectAllowed(Value.Get()))
	{
		SerializeStringInternal(FString::Printf(TEXT("Lazy:%s"), *Value.GetUniqueID().ToString()));
	}
	else
	{
		WriteValue(TEXT("null"));
	}
}

void FJsonArchiveOutputFormatterEx::Serialize(FObjectPtr& Value)
{
	UObject* Object = Value.Get();
	Serialize(Object);
}

void FJsonArchiveOutputFormatterEx::Serialize(TArray<uint8>& Data)
{
	Serialize(Data.GetData(), Data.Num());
}

void FJsonArchiveOutputFormatterEx::Serialize(void* Data, uint64 DataSize)
{
	static const int32 MaxLineChars = 120;
	static const int32 MaxLineBytes = FBase64::GetMaxDecodedDataSize(MaxLineChars);

	if(DataSize < MaxLineBytes)
	{
		// Encode the data on a single line. No need for hashing; intra-line merge conflicts are rare.
		WriteValue(FString::Printf(TEXT("\"Base64:%s\""), *FBase64::Encode((const uint8*)Data, DataSize)));
	}
	else
	{
		// Encode the data as a record containing a digest and array of base-64 encoded lines
		EnterRecord();
		Inner.Serialize(Newline.GetData(), Newline.Num());

		// Compute a SHA digest for the raw data, so we can check if it's corrupted
		uint8 Digest[FSHA1::DigestSize];
		FSHA1::HashBuffer(Data, DataSize, Digest);

		// Convert the hash to a string
		ANSICHAR DigestString[(FSHA1::DigestSize * 2) + 1];
		for(int32 Idx = 0; Idx < UE_ARRAY_COUNT(Digest); Idx++)
		{
			static const ANSICHAR HexDigits[] = "0123456789abcdef";
			DigestString[(Idx * 2) + 0] = HexDigits[Digest[Idx] >> 4];
			DigestString[(Idx * 2) + 1] = HexDigits[Digest[Idx] & 15];
		}
		DigestString[UE_ARRAY_COUNT(DigestString) - 1] = 0;

		// Write the digest
		Write("\"Digest\": \"");
		Write(DigestString);
		Write("\",");
		Inner.Serialize(Newline.GetData(), Newline.Num());

		// Write the base64 data
		Write("\"Base64\": ");
		for(uint64 DataPos = 0; DataPos < DataSize; DataPos += MaxLineBytes)
		{
			Write((DataPos > 0)? ',' : '[');
			Inner.Serialize(Newline.GetData(), Newline.Num());
			Write("\t\"");

			ANSICHAR LineData[MaxLineChars + 1];
			uint64 NumLineChars = FBase64::Encode((const uint8*)Data + DataPos, FMath::Min<uint64>(DataSize - DataPos, MaxLineBytes), LineData);
			Inner.Serialize(LineData, NumLineChars);

			Write("\"");
		}

		// Close the array
		Inner.Serialize(Newline.GetData(), Newline.Num());
		Write(']');
		bNeedsNewline = true;

		// Close the record
		LeaveRecord();
	}
}

void FJsonArchiveOutputFormatterEx::Write(ANSICHAR Character)
{
	Inner.Serialize((void*)&Character, 1);
}

void FJsonArchiveOutputFormatterEx::Write(const ANSICHAR* Text)
{
	Inner.Serialize((void*)Text, TCString<ANSICHAR>::Strlen(Text));
}

void FJsonArchiveOutputFormatterEx::Write(const FString& Text)
{
	Write(TCHAR_TO_UTF8(*Text));
}

void FJsonArchiveOutputFormatterEx::WriteFieldName(const TCHAR* Name)
{
	if(FCString::Stricmp(Name, TEXT("Base64")) == 0 || FCString::Stricmp(Name, TEXT("Digest")) == 0)
	{
		Write(FString::Printf(TEXT("\"_%s\": "), Name));
	}
	else if(Name[0] == '_')
	{
		Write(FString::Printf(TEXT("\"_%s\": "), Name));
	}
	else
	{
		Write(FString::Printf(TEXT("\"%s\": "), Name));
	}
}

void FJsonArchiveOutputFormatterEx::WriteValue(const FString& Text)
{
	Write(Text);
}

void FJsonArchiveOutputFormatterEx::WriteOptionalComma()
{
	if (bNeedsComma)
	{
		Write(',');
		bNeedsComma = false;
	}
}

void FJsonArchiveOutputFormatterEx::WriteOptionalNewline()
{
	if (bNeedsNewline)
	{
		Inner.Serialize(Newline.GetData(), Newline.Num());
		bNeedsNewline = false;
	}
}

void FJsonArchiveOutputFormatterEx::WriteOptionalAttributedBlockOpening()
{
	if (NumAttributesStack.Top() == 0)
	{
		Write('{');
		Newline.Add('\t');
		bNeedsNewline = true;
	}
}

void FJsonArchiveOutputFormatterEx::WriteOptionalAttributedBlockValue()
{
	if (NumAttributesStack.Top() != 0)
	{
		WriteFieldName(TEXT("_Value"));
	}
}

void FJsonArchiveOutputFormatterEx::WriteOptionalAttributedBlockClosing()
{
	if (NumAttributesStack.Top() != 0)
	{
		Newline.Pop(EAllowShrinking::No);
		WriteOptionalNewline();
		Write("}");
		bNeedsComma                  = true;
		bNeedsNewline                = true;
	}
}

void FJsonArchiveOutputFormatterEx::SerializeStringInternal(const FString& String)
{
	FString Result = TEXT("\"");

	// Escape the string characters
	for (int32 Idx = 0; Idx < String.Len(); Idx++)
	{
		switch (String[Idx])
		{
		case '\"':
			Result += "\\\"";
			break;
		case '\\':
			Result += "\\\\";
			break;
		case '\b':
			Result += "\\b";
			break;
		case '\f':
			Result += "\\f";
			break;
		case '\n':
			Result += "\\n";
			break;
		case '\r':
			Result += "\\r";
			break;
		case '\t':
			Result += "\\t";
			break;
		default:
			if (String[Idx] <= 0x1f || String[Idx] >= 0x7f)
			{
				Result += FString::Printf(TEXT("\\u%04x"), String[Idx]);
			}
			else
			{
				Result.AppendChar(String[Idx]);
			}
			break;
		}
	}
	Result += TEXT("\"");

	WriteValue(Result);
}

bool FJsonArchiveOutputFormatterEx::IsObjectAllowed(UObject* InObject) const
{
	return ObjectIndicesMap && ObjectIndicesMap->Contains(InObject);
}

#endif
