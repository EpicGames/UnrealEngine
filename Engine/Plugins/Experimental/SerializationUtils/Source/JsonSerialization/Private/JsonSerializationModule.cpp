// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonSerializationModule.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogJsonSerialization);

void FJsonSerializationModule::StartupModule()
{
}

void FJsonSerializationModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FJsonSerializationModule, JsonSerialization)