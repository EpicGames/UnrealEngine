// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Dom/JsonObject.h"
#include "HAL/Platform.h"
#include "Serialization/StructuredArchiveFormatter.h"
#include "Serialization/StructuredArchiveNameHelpers.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectResource.h"

class FArchive;
class FJsonObject;
class FJsonValue;
class FName;
class FPackageIndex;
class FText;
class UObject;
struct FLazyObjectPtr;
struct FObjectPtr;
struct FSoftObjectPath;
struct FSoftObjectPtr;
struct FWeakObjectPtr;

#if WITH_TEXT_ARCHIVE_SUPPORT

class JSONSERIALIZATION_API FJsonArchiveInputFormatterEx final : public FStructuredArchiveFormatter
{
public:
	FJsonArchiveInputFormatterEx(FArchive& InInner, UObject* InRootObject, TFunction<UObject* (const FPackageIndex)> InResolveObject = nullptr);
	virtual ~FJsonArchiveInputFormatterEx() override;

	virtual FArchive& GetUnderlyingArchive() override;
	virtual FStructuredArchiveFormatter* CreateSubtreeReader() override;

	virtual bool HasDocumentTree() const override;

	virtual void EnterRecord() override;
	virtual void LeaveRecord() override;
	virtual void EnterField(FArchiveFieldName Name) override;
	virtual void LeaveField() override;
	virtual bool TryEnterField(FArchiveFieldName Name, bool bEnterWhenSaving) override;

	virtual void EnterArray(int32& NumElements) override;
	virtual void LeaveArray() override;
	virtual void EnterArrayElement() override;
	virtual void LeaveArrayElement() override;

	virtual void EnterStream() override;
	virtual void LeaveStream() override;
	virtual void EnterStreamElement() override;
	virtual void LeaveStreamElement() override;

	virtual void EnterMap(int32& NumElements) override;
	virtual void LeaveMap() override;
	virtual void EnterMapElement(FString& Name) override;
	virtual void LeaveMapElement() override;

	virtual void EnterAttributedValue() override;
	virtual void EnterAttribute(FArchiveFieldName AttributeName) override;
	virtual void EnterAttributedValueValue() override;
	virtual void LeaveAttribute() override;
	virtual void LeaveAttributedValue() override;
	virtual bool TryEnterAttribute(FArchiveFieldName AttributeName, bool bEnterWhenSavin) override;
	virtual bool TryEnterAttributedValueValue() override;

	virtual void Serialize(uint8& Value) override;
	virtual void Serialize(uint16& Value) override;
	virtual void Serialize(uint32& Value) override;
	virtual void Serialize(uint64& Value) override;
	virtual void Serialize(int8& Value) override;
	virtual void Serialize(int16& Value) override;
	virtual void Serialize(int32& Value) override;
	virtual void Serialize(int64& Value) override;
	virtual void Serialize(float& Value) override;
	virtual void Serialize(double& Value) override;
	virtual void Serialize(bool& Value) override;
	virtual void Serialize(UTF32CHAR& Value) override;
	virtual void Serialize(FString& Value) override;
	virtual void Serialize(FName& Value) override;
	virtual void Serialize(UObject*& Value) override;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	virtual void Serialize(Verse::VCell*& Value) override;
#endif
	virtual void Serialize(FText& Value) override;
	virtual void Serialize(FWeakObjectPtr& Value) override;
	virtual void Serialize(FSoftObjectPtr& Value) override;
	virtual void Serialize(FSoftObjectPath& Value) override;
	virtual void Serialize(FLazyObjectPtr& Value) override;
	virtual void Serialize(FObjectPtr& Value) override;
	virtual void Serialize(TArray<uint8>& Value) override;
	virtual void Serialize(void* Data, uint64 DataSize) override;

private:
	bool IsNestedObject();
	UObject* LoadNestedObject();
	
	FArchive& Inner;

	struct FObjectRecord
	{
		FObjectRecord(TSharedPtr<FJsonObject> InJsonObject, int64 InValueCount)
			: JsonObject(InJsonObject)
			, ValueCountOnCreation(InValueCount)
		{

		}

		TSharedPtr<FJsonObject> JsonObject;
		int64 ValueCountOnCreation;	// For debugging purposes, so we can ensure all values have been consumed
	};

	TFunction<UObject* (const FPackageIndex)> ResolveObject;
	UObject* RootObject;
	TArray<FObjectRecord> ObjectStack;
	TArray<TSharedPtr<FJsonValue>> ValueStack;
	TArray<TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>::TIterator> MapIteratorStack;
	TArray<int32> ArrayValuesRemainingStack;

	static FString EscapeFieldName(const TCHAR* Name);
	static FString UnescapeFieldName(const TCHAR* Name);

	static EArchiveValueType GetValueType(const FJsonValue &Value);
};

#endif
