// Copyright Epic Games, Inc. All Rights Reserved.

#include "Backends/XmlStructSerializerBackend.h"

#include "UObject/StrProperty.h"
#include "XmlSerializationModule.h"
#include "UObject/EnumProperty.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"
#include "Utils/XmlUtils.h"
#include "Utils/XmlWriter.h"

#include "pugixml.hpp"

class FXmlStructSerializerBackend::FImpl
{
public:
	FImpl()
	{
		pugi::xml_node decl = XmlDoc.prepend_child(pugi::node_declaration);
		decl.append_attribute("version") = "1.0";
		decl.append_attribute("encoding") = "utf-8";

		XmlElement = XmlDoc.append_child();
		XmlElement.set_name("StructFileDocument");
		XmlElement.append_attribute(("Version")) = "1.0";
	}
	
	pugi::xml_document XmlDoc;
	pugi::xml_node XmlElement;
	TUniquePtr<FXmlWriter> XmlWriter;
};

FXmlStructSerializerBackend::FXmlStructSerializerBackend(FArchive& InArchive,
                                                         const EStructSerializerBackendFlags InFlags)
	: Impl(new FImpl)
	  , Flags(InFlags)
{
	Impl->XmlWriter = MakeUnique<FXmlWriter>(&InArchive);
}

FXmlStructSerializerBackend::~FXmlStructSerializerBackend()
{
	delete Impl;
}

void FXmlStructSerializerBackend::SaveDocument(EXmlSerializationEncoding InEncoding)
{
	using namespace UE::XmlSerialization::Private;
	const pugi::xml_encoding encoding = Utils::ToPugiEncoding(InEncoding);
	Impl->XmlDoc.save(*Impl->XmlWriter, "  ", pugi::format_default, encoding);
}

void FXmlStructSerializerBackend::BeginArray(const FStructSerializerState& InState)
{
	pugi::xml_node ArrayElement = Impl->XmlElement.append_child(pugi::node_element);
	ArrayElement.set_name("Array");
	if (InState.ValueProperty)
	{
		ArrayElement.append_attribute("Name") = STRING_CAST_UE_TO_PUGI(*InState.ValueProperty->GetName());
		ArrayElement.append_attribute("Type") = STRING_CAST_UE_TO_PUGI(*InState.ValueProperty->GetClass()->GetName());
	}
	if (InState.ValueType)
	{
		ArrayElement.append_attribute("ValueType") = STRING_CAST_UE_TO_PUGI(*InState.ValueType->GetName());
	}
	Impl->XmlElement = ArrayElement;
}

void FXmlStructSerializerBackend::BeginStructure(const FStructSerializerState& InState)
{
	pugi::xml_node StructElement = Impl->XmlElement.append_child(pugi::node_element);
	StructElement.set_name("Struct");
	if (InState.ValueProperty)
	{
		StructElement.append_attribute("Name") = STRING_CAST_UE_TO_PUGI(*InState.ValueProperty->GetName());
		StructElement.append_attribute("Type") = STRING_CAST_UE_TO_PUGI(*InState.ValueProperty->GetClass()->GetName());
	}
	if (InState.ValueType)
	{
		StructElement.append_attribute("ValueType") = STRING_CAST_UE_TO_PUGI(*InState.ValueType->GetName());
	}
	Impl->XmlElement = StructElement;
}


void FXmlStructSerializerBackend::EndArray(const FStructSerializerState& /*State*/)
{
	// Check that current element is an Array?
	Impl->XmlElement = Impl->XmlElement.parent();
}


void FXmlStructSerializerBackend::EndStructure(const FStructSerializerState& /*State*/)
{
	// Check that current element is a Struct?
	Impl->XmlElement = Impl->XmlElement.parent();
}

void FXmlStructSerializerBackend::WriteComment(const FString& InComment)
{
	pugi::xml_node Child = Impl->XmlElement.append_child(pugi::node_comment);
	Child.set_value(STRING_CAST_UE_TO_PUGI(*InComment));
}

void FXmlStructSerializerBackend::WriteProperty(const FStructSerializerState& InState, int32 InArrayIndex)
{
	// Append a property element in the current structure element.
	pugi::xml_node PropertyElement = Impl->XmlElement.append_child(pugi::node_element);
	PropertyElement.set_name("Property");
	PropertyElement.append_attribute("Name") = STRING_CAST_UE_TO_PUGI(*InState.ValueProperty->GetName());
	PropertyElement.append_attribute("Type") = STRING_CAST_UE_TO_PUGI(*InState.ValueProperty->GetClass()->GetName());
	if (InState.ValueType)
	{
		PropertyElement.append_attribute("ValueType") = STRING_CAST_UE_TO_PUGI(*InState.ValueType->GetName());
	}
	pugi::xml_attribute ValueAttribute = PropertyElement.append_attribute("Value");

	// booleans
	if (InState.FieldType == FBoolProperty::StaticClass())
	{
		ValueAttribute.set_value(CastFieldChecked<FBoolProperty>(InState.ValueProperty)->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex));
	}
	
	// unsigned bytes & enumerations
	else if (InState.FieldType == FEnumProperty::StaticClass())
	{
		FEnumProperty* EnumProperty = CastFieldChecked<FEnumProperty>(InState.ValueProperty);

		ValueAttribute.set_value(STRING_CAST_UE_TO_PUGI(*EnumProperty->GetEnum()->GetNameStringByValue(EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(EnumProperty->ContainerPtrToValuePtr<void>(InState.ValueData, InArrayIndex)))));
	}
	else if (InState.FieldType == FByteProperty::StaticClass())
	{
		FByteProperty* ByteProperty = CastFieldChecked<FByteProperty>(InState.ValueProperty);

		if (ByteProperty->IsEnum())
		{
			ValueAttribute.set_value(STRING_CAST_UE_TO_PUGI(*ByteProperty->Enum->GetNameStringByValue(ByteProperty->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex))));
		}
		else
		{
			ValueAttribute.set_value((int)ByteProperty->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex));
		}
	}
	
	// floating point numbers
	else if (InState.FieldType == FDoubleProperty::StaticClass())
	{
		ValueAttribute.set_value(CastFieldChecked<FDoubleProperty>(InState.ValueProperty)->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex));
	}
	else if (InState.FieldType == FFloatProperty::StaticClass())
	{
		ValueAttribute.set_value(CastFieldChecked<FFloatProperty>(InState.ValueProperty)->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex));
	}

	// signed integers
	else if (InState.FieldType == FIntProperty::StaticClass())
	{
		ValueAttribute.set_value((int)CastFieldChecked<FIntProperty>(InState.ValueProperty)->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex));
	}
	else if (InState.FieldType == FInt8Property::StaticClass())
	{
		ValueAttribute.set_value((int)CastFieldChecked<FInt8Property>(InState.ValueProperty)->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex));
	}
	else if (InState.FieldType == FInt16Property::StaticClass())
	{
		ValueAttribute.set_value((int)CastFieldChecked<FInt16Property>(InState.ValueProperty)->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex));
	}
	else if (InState.FieldType == FInt64Property::StaticClass())
	{
		ValueAttribute.set_value((long long)CastFieldChecked<FInt64Property>(InState.ValueProperty)->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex));
	}
	
	// unsigned integers
	else if (InState.FieldType == FUInt16Property::StaticClass())
	{
		ValueAttribute.set_value((unsigned int)CastFieldChecked<FUInt16Property>(InState.ValueProperty)->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex));
	}
	else if (InState.FieldType == FUInt32Property::StaticClass())
	{
		ValueAttribute.set_value((unsigned int)CastFieldChecked<FUInt32Property>(InState.ValueProperty)->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex));
	}
	else if (InState.FieldType == FUInt64Property::StaticClass())
	{
		ValueAttribute.set_value((unsigned long long)CastFieldChecked<FUInt64Property>(InState.ValueProperty)->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex));
	}

	// names, strings & text
	else if (InState.FieldType == FNameProperty::StaticClass())
	{
		ValueAttribute.set_value(STRING_CAST_UE_TO_PUGI(*CastFieldChecked<FNameProperty>(InState.ValueProperty)->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex).ToString()));
	}
	else if (InState.FieldType == FStrProperty::StaticClass())
	{
		ValueAttribute.set_value(STRING_CAST_UE_TO_PUGI(*CastFieldChecked<FStrProperty>(InState.ValueProperty)->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex)));
	}
	else if (InState.FieldType == FTextProperty::StaticClass())
	{
		const FText& TextValue = CastFieldChecked<FTextProperty>(InState.ValueProperty)->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex);
		if (EnumHasAnyFlags(Flags, EStructSerializerBackendFlags::WriteTextAsComplexString))
		{
			FString TextValueString;
			FTextStringHelper::WriteToBuffer(TextValueString, TextValue);
			ValueAttribute.set_value(STRING_CAST_UE_TO_PUGI(*TextValueString));
		}
		else
		{
			ValueAttribute.set_value(STRING_CAST_UE_TO_PUGI(*TextValue.ToString()));
		}
	}

	// classes & objects
	else if (InState.FieldType == FSoftClassProperty::StaticClass())
	{
		FSoftObjectPtr const& Value = CastFieldChecked<FSoftClassProperty>(InState.ValueProperty)->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex);
		ValueAttribute.set_value(STRING_CAST_UE_TO_PUGI(*(Value.IsValid() ? Value->GetPathName() : FString())));
	}
	else if (InState.FieldType == FWeakObjectProperty::StaticClass())
	{
		FWeakObjectPtr const& Value = CastFieldChecked<FWeakObjectProperty>(InState.ValueProperty)->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex);
		ValueAttribute.set_value(STRING_CAST_UE_TO_PUGI(*(Value.IsValid() ? Value.Get()->GetPathName() : FString())));
	}
	else if (InState.FieldType == FSoftObjectProperty::StaticClass())
	{
		FSoftObjectPtr const& Value = CastFieldChecked<FSoftObjectProperty>(InState.ValueProperty)->GetPropertyValue_InContainer(InState.ValueData, InArrayIndex);
		ValueAttribute.set_value(STRING_CAST_UE_TO_PUGI(*Value.ToString()));
	}
	else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InState.ValueProperty))
	{
		// @TODO: Could this be expanded to include everything derived from FObjectPropertyBase?
		// Generic handling for a property type derived from FObjectProperty that is obtainable as a pointer and will be stored using its path.
		// This must come after all the more specialized handlers for object property types.
		UObject* const Value = ObjectProperty->GetObjectPropertyValue_InContainer(InState.ValueData, InArrayIndex);
		ValueAttribute.set_value(STRING_CAST_UE_TO_PUGI(*(Value ? Value->GetPathName() : FString())));
	}
	// unsupported property type
	else
	{
		const FString ValueType = InState.ValueType != nullptr ? InState.ValueType->GetFName().ToString() : TEXT("unknown");
		UE_LOGF(LogXmlSerialization, Verbose, "FXmlStructSerializerBackend: Property %ls cannot be serialized, because its type (%ls) is not supported"
			, *InState.ValueProperty->GetFName().ToString()
			, *ValueType);
	}
}
