// Copyright Epic Games, Inc. All Rights Reserved.

#include "Formatters/XmlArchiveInputFormatter.h"
#include "Misc/Base64.h"
#include "Misc/SecureHash.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/ObjectPtr.h"

#include "Logging/LogMacros.h"
#include "Utils/XmlWriter.h"
#include "Utils/XmlUtils.h"
#include "XmlFormatterHelper.h"
#include "XmlSerializationModule.h"
#include "UObject/UObjectIterator.h"

#if WITH_TEXT_ARCHIVE_SUPPORT

// Private Implementation
class FXmlArchiveInputFormatter::FImpl
{
public:
	FImpl()
	{
		XmlElements.Reserve(64);
	}
	
	// Copy constructor is used when making a sub-tree. Not Tested.
	FImpl(const FImpl& InOther)
	{
		XmlDocument.reset(InOther.XmlDocument);
		XmlParseResult = InOther.XmlParseResult;
		const std::string NodePath = InOther.XmlElements.Top().path();
		XmlElements.Push(XmlDocument.first_element_by_path(NodePath.c_str()));
	}

	bool Pop(const TCHAR* InFuncName, const TCHAR* InTagName)
	{
		using namespace UE::XmlSerialization::Private;
		
		// Check if the tag name matches.
		if (!Utils::IsElementA(XmlElements.Top(), InTagName))
		{
			UE_LOGF(LogXmlSerialization, Warning,
				"FXmlArchiveInputFormatter::%ls: Requested to leave \"%ls\", but it is \"%ls\".",
				InFuncName, InTagName, *Utils::GetInfo(XmlElements.Top()));
		}
		
		XmlElements.Pop();
		return true;
	}

	void Push(const pugi::xml_node& InXmlNode)
	{
		XmlElements.Push(InXmlNode);
	}

	pugi::xml_attribute GetValueAttrib() const
	{
		using namespace UE::XmlSerialization::Private;
		const pugi::xml_attribute Attrib = !XmlAttrib.empty() ? XmlAttrib : XmlElement().attribute("Value");
		if (Attrib.empty())
		{
			UE_LOGF(LogXmlSerialization, Error,
				"FXmlArchiveInputFormatter::Serialize: No current attribute. Current Node: \"%ls\"",
				*Utils::GetInfo(XmlElement()));
		}
		return Attrib;
	}

	/** Current xml element being traversed. */
	const pugi::xml_node& XmlElement() const { return XmlElements.Top(); }

	pugi::xml_node& XmlElementMutable() { return XmlElements.Top(); }
	
	/** Current Document. */
	pugi::xml_document XmlDocument;

	/** Parse result of parsing the document. */
	pugi::xml_parse_result XmlParseResult;

	/**
	 * Stack of xml element being traversed.
	 * Using a stack so we can push empty nodes and pop them back up to the previous existing parent.
	 * This allows traversing missing tags and recover.
	 */
	TArray<pugi::xml_node> XmlElements;

	/** Current Attribute being traversed. */
	pugi::xml_attribute XmlAttrib;

	/** Last Array/Stream/Map element that was traversed. */
	pugi::xml_node LastArrayElement;
};

FXmlArchiveInputFormatter::FXmlArchiveInputFormatter(FArchive& InInner, UObject* InRootObject, const TFunction<UObject* (const FPackageIndex)>& InResolveObject) 
	: Impl(new FImpl)
	, Inner(InInner)
	, ResolveObject(InResolveObject)
	, RootObject(InRootObject)
{
	Inner.SetIsTextFormat(true);
	Inner.ArAllowLazyLoading = false;

	{
		// Read the whole thing in memory and parse it.
		TArray<int8> Buffer;
		Buffer.SetNum(Inner.TotalSize());
		Inner.Serialize(Buffer.GetData(), Buffer.Num());
		Impl->XmlParseResult = Impl->XmlDocument.load_buffer(Buffer.GetData(), Buffer.Num(),
		                                                              pugi::parse_default | pugi::parse_declaration |
		                                                              pugi::parse_comments);
		if (Impl->XmlParseResult.status != pugi::status_ok)
		{
			//Impl->XmlParseResult.description() -> convert to FString.
			UE_LOGF(LogXmlSerialization, Error, "FXmlArchiveInputFormatter: Failed to parse xml document."); 
		}
		else
		{
			// Find the root of the document.
			pugi::xml_node RootNode = Impl->XmlDocument.document_element();
			if (FCStringAnsi::Strcmp(RootNode.name(), "ObjectFileDocument") != 0)
			{
				RootNode = RootNode.next_sibling("ObjectFileDocument");
				if (RootNode.empty())
				{
					RootNode = Impl->XmlDocument.document_element().child("ObjectFileDocument");
				}
			}
			
			if (RootNode.empty())
			{
				UE_LOGF(LogXmlSerialization, Error, "FXmlArchiveInputFormatter: Document is missing the root tag."); 
			}
			
			Impl->Push(RootNode);
		}
	}
}

FXmlArchiveInputFormatter::FXmlArchiveInputFormatter(const FXmlArchiveInputFormatter& InOther)
	: Impl(new FImpl(*InOther.Impl))
	, Inner(InOther.Inner)
	, ResolveObject(InOther.ResolveObject)
	, RootObject(InOther.RootObject)
{
	
}

FXmlArchiveInputFormatter::~FXmlArchiveInputFormatter()
{
}

bool FXmlArchiveInputFormatter::IsParseResultStatusOk() const
{
	return (Impl->XmlParseResult.status == pugi::status_ok);
}

FArchive& FXmlArchiveInputFormatter::GetUnderlyingArchive()
{
	return Inner;
}

FStructuredArchiveFormatter* FXmlArchiveInputFormatter::CreateSubtreeReader()
{
	FXmlArchiveInputFormatter* Cloned = new FXmlArchiveInputFormatter(*this);
	// TODO: not sure this works.
	Cloned->Impl->XmlDocument.append_copy(Impl->XmlElement());
	Cloned->Impl->Push(Cloned->Impl->XmlDocument.document_element());
	return Cloned;
}

bool FXmlArchiveInputFormatter::HasDocumentTree() const
{
	return true;
}

void FXmlArchiveInputFormatter::EnterRecord()
{
	using namespace UE::XmlSerialization::Private;
	const pugi::xml_node RecordElement = Impl->XmlElement().child("Record");
	if (RecordElement.empty())
	{
		UE_LOGF(LogXmlSerialization, Warning,
		       "FXmlArchiveInputFormatter::EnterRecord: Node \"%ls\" has no \"Record\" child.",
		       		*Utils::GetInfo(Impl->XmlElement()));
	}
	
	Impl->Push(RecordElement);
}

void FXmlArchiveInputFormatter::LeaveRecord()
{
	Impl->Pop(TEXT("LeaveRecord"), TEXT("Record"));
}

void FXmlArchiveInputFormatter::EnterField(FArchiveFieldName InName)
{
	using namespace UE::XmlSerialization::Private;
	// We are looking for a "Field" node with the attribute "Name" being the given name.
	// Note: this only searches by attribute name/value.
	const pugi::xml_node FieldElement = Impl->XmlElement().find_child_by_attribute("Field", "Name", STRING_CAST_UE_TO_PUGI(InName.Name));
	if (FieldElement.empty())
	{
		UE_LOGF(LogXmlSerialization, Warning,
		       "FXmlArchiveInputFormatter::EnterField: Failed to find Field \"%ls\". Current Node: \"%ls\".", InName.Name,
		       *Utils::GetInfo(Impl->XmlElement()));
	}
	
	Impl->Push(FieldElement);
}

void FXmlArchiveInputFormatter::LeaveField()
{
	Impl->Pop(TEXT("LeaveField"), TEXT("Field"));
}

bool FXmlArchiveInputFormatter::TryEnterField(FArchiveFieldName InName, bool bInEnterWhenSaving)
{
	const pugi::xml_node FieldElement = Impl->XmlElement().find_child_by_attribute("Field", "Name", STRING_CAST_UE_TO_PUGI(InName.Name));
	if (FieldElement.empty())
	{
		return false; 
	}
	
	Impl->Push(FieldElement);
	return true;
}

void FXmlArchiveInputFormatter::EnterArray(int32& OutNumElements)
{
	using namespace UE::XmlSerialization::Private;
	const pugi::xml_node ArrayElement = Impl->XmlElement().child("Array");
	
	if (!ArrayElement.empty())
	{
		const pugi::xml_attribute SizeAttrib = ArrayElement.attribute("Size");
		if (SizeAttrib.empty())
		{
			UE_LOGF(LogXmlSerialization, Error,
				"FXmlArchiveInputFormatter::EnterArray: Node \"%ls\" doesn't have a \"Size\" attribute.",
				*Utils::GetInfo(Impl->XmlElement())); 
		}
	
		OutNumElements = !SizeAttrib.empty() ? SizeAttrib.as_int() : 0;
	}
	else
	{
		OutNumElements = 0;
		
		UE_LOGF(LogXmlSerialization, Error,
			"FXmlArchiveInputFormatter::EnterArray: Node \"%ls\" doesn't have an \"Array\" child.",
			*Utils::GetInfo(Impl->XmlElement())); 
	}
	
	Impl->LastArrayElement = pugi::xml_node();
	Impl->Push(ArrayElement);
}

void FXmlArchiveInputFormatter::LeaveArray()
{
	Impl->Pop(TEXT("LeaveArray"), TEXT("Array"));
	Impl->LastArrayElement = pugi::xml_node();
}

void FXmlArchiveInputFormatter::EnterArrayElement()
{
	using namespace UE::XmlSerialization::Private;
	pugi::xml_node ArrayElementElement;
	
	if (!Impl->LastArrayElement.empty())
	{
		ArrayElementElement = Impl->LastArrayElement.next_sibling("ArrayElement");
		
		if (ArrayElementElement.empty())
		{
			UE_LOGF(LogXmlSerialization, Error,
				"FXmlArchiveInputFormatter::EnterArrayElement: Node \"%ls\" doesn't have a \"ArrayElement\" sibling.",
				*Utils::GetInfo(Impl->LastArrayElement)); 
		}
	}
	else
	{
		ArrayElementElement = Impl->XmlElement().child("ArrayElement");
		
		if (ArrayElementElement.empty())
		{
			UE_LOGF(LogXmlSerialization, Error,
				"FXmlArchiveInputFormatter::EnterArrayElement: Node \"%ls\" doesn't have a \"ArrayElement\" child.",
				*Utils::GetInfo(Impl->XmlElement())); 
		}
	}

	Impl->Push(ArrayElementElement);
}

void FXmlArchiveInputFormatter::LeaveArrayElement()
{
	using namespace UE::XmlSerialization::Private;
	Impl->LastArrayElement =  Utils::IsElementA(Impl->XmlElement(),TEXT("ArrayElement")) ? Impl->XmlElement() : pugi::xml_node();
	Impl->Pop(TEXT("LeaveArrayElement"), TEXT("ArrayElement"));
}

void FXmlArchiveInputFormatter::EnterStream()
{
	using namespace UE::XmlSerialization::Private;
	const pugi::xml_node StreamElement = Impl->XmlElement().child("Stream");

	if (StreamElement.empty())
	{
		UE_LOGF(LogXmlSerialization, Error,
			"FXmlArchiveInputFormatter::EnterStream: Node \"%ls\" doesn't have a \"Stream\" child.",
			*Utils::GetInfo(Impl->XmlElement())); 
	}
	
	Impl->Push(StreamElement);
	Impl->LastArrayElement = pugi::xml_node();
}

void FXmlArchiveInputFormatter::LeaveStream()
{
	Impl->Pop(TEXT("LeaveStream"), TEXT("Stream"));
	Impl->LastArrayElement = pugi::xml_node();
}

void FXmlArchiveInputFormatter::EnterStreamElement()
{
	using namespace UE::XmlSerialization::Private;
	pugi::xml_node StreamElementElement;
	
	if (!Impl->LastArrayElement.empty())
	{
		StreamElementElement = Impl->LastArrayElement.next_sibling("StreamElement");

		if (!StreamElementElement.empty())
		{
			UE_LOGF(LogXmlSerialization, Error,
				"FXmlArchiveInputFormatter::EnterStreamElement: Node \"%ls\" doesn't have a \"StreamElement\" sibling.",
				*Utils::GetInfo(Impl->LastArrayElement));
		}
	}
	else
	{
		StreamElementElement = Impl->XmlElement().child("StreamElement");

		if (!StreamElementElement.empty())
		{
			UE_LOGF(LogXmlSerialization, Error,
				"FXmlArchiveInputFormatter::EnterStreamElement: Node \"%ls\" doesn't have a \"StreamElement\" child.",
				*Utils::GetInfo(Impl->XmlElement()));
		}
	}
	
	Impl->Push(StreamElementElement);
}

void FXmlArchiveInputFormatter::LeaveStreamElement()
{
	using namespace UE::XmlSerialization::Private;
	Impl->LastArrayElement =  Utils::IsElementA(Impl->XmlElement(),TEXT("StreamElement")) ? Impl->XmlElement() : pugi::xml_node();
	Impl->Pop(TEXT("LeaveStreamElement"), TEXT("StreamElement"));
}

void FXmlArchiveInputFormatter::EnterMap(int32& OutNumElements)
{
	using namespace UE::XmlSerialization::Private;
	const pugi::xml_node MapElement = Impl->XmlElement().child("Map");
	if (!MapElement.empty())
	{
		const pugi::xml_attribute SizeAttrib = MapElement.attribute("Size");

		if (SizeAttrib.empty())
		{
			UE_LOGF(LogXmlSerialization, Error, "FXmlArchiveInputFormatter::EnterMap: Node \"%s\" doesn't have a \"Size\" attribute.", MapElement.name()); 
		}

		OutNumElements = !SizeAttrib.empty() ? SizeAttrib.as_int() : 0;
		
		Impl->Push(MapElement);
		Impl->LastArrayElement = pugi::xml_node();
		return;
	}
	
	// Fallback mode: There might be a bug on how MapProperties (or Struct?) are serialized. The code is different for loading vs saving.
	// See: UStruct::SerializeVersionedTaggedProperties. When loading it wants to enter a map, while when saving, it enters a record.
	pugi::xml_node RecordElement = Impl->XmlElement().child("Record");
	if (!RecordElement.empty())
	{
		// Remember we where supposed to get into a Map.
		RecordElement.append_attribute("__HACK_SHOULD_BE__") = "Map"; 

		// The number of elements are going to be the number of Field nodes.
		// It will then try to enter MapElements, but we'll be looking for Field Nodes instead.
		OutNumElements = 0;
		for (pugi::xml_node Field : RecordElement.children("Field"))
		{
			OutNumElements++;
		}	

		UE_LOGF(LogXmlSerialization, Verbose, "FXmlArchiveInputFormatter:EnterMap: Fallback: Entering \"Record\" node instead, with %d entries (Fields).", OutNumElements);
		Impl->Push(RecordElement);
		Impl->LastArrayElement = pugi::xml_node();
		return;
	}

	UE_LOGF(LogXmlSerialization, Error,
		"FXmlArchiveInputFormatter::EnterMap: Node \"%ls\" doesn't have a \"Map\" (nor \"Record\") child.",
		*Utils::GetInfo(Impl->XmlElement()));
	
	OutNumElements = 0;
	Impl->Push(pugi::xml_node());
	Impl->LastArrayElement = pugi::xml_node();
}

inline bool ElementIsARecordButShouldBeAMap(const pugi::xml_node& InElement)
{
	if (FCStringAnsi::Strcmp(InElement.name(), "Record") == 0)
	{
		const pugi::xml_attribute HackAttrib = InElement.attribute("__HACK_SHOULD_BE__");
		return (!HackAttrib.empty() && FCStringAnsi::Strcmp(HackAttrib.as_string(), "Map") == 0) ? true : false;
	}
	return false;
}

void FXmlArchiveInputFormatter::LeaveMap()
{
	if (ElementIsARecordButShouldBeAMap(Impl->XmlElement()))
	{
		Impl->Pop(TEXT("LeaveMap"), TEXT("Record"));
	}
	else
	{
		Impl->Pop(TEXT("LeaveMap"), TEXT("Map"));
	}
	Impl->LastArrayElement = pugi::xml_node();
}

void FXmlArchiveInputFormatter::EnterMapElement(FString& OutName)
{
	using namespace UE::XmlSerialization::Private;
	
	// Fallback code for the Map vs Record bug.
	if (ElementIsARecordButShouldBeAMap(Impl->XmlElement()))
	{
		pugi::xml_node FieldElement;
		
		if (!Impl->LastArrayElement.empty())
		{
			FieldElement = Impl->LastArrayElement.next_sibling("Field");
			
			if (FieldElement.empty())
			{
				UE_LOGF(LogXmlSerialization, Error,
				   "FXmlArchiveInputFormatter::EnterMapElement:(Fallback) \"%ls\" node doesn't have a \"Field\" sibling, which was supposed to be a \"MapElement\".",
					*Utils::GetInfo(Impl->LastArrayElement));
			}
		}
		else
		{
			FieldElement = Impl->XmlElement().child("Field");

			if (FieldElement.empty())
			{
				UE_LOGF(LogXmlSerialization, Error,
				   "FXmlArchiveInputFormatter::EnterMapElement:(Fallback) \"%ls\" node doesn't have a \"Field\" child, which was supposed to be a \"MapElement\".",
					*Utils::GetInfo(Impl->XmlElement()));
			}
		}

		if (!FieldElement.empty())
		{
			const pugi::xml_attribute NameAttrib = FieldElement.attribute("Name");
			OutName = NameAttrib.as_string();
			if (NameAttrib.empty())
			{
				UE_LOGF(LogXmlSerialization, Error,
					"FXmlArchiveInputFormatter::EnterMapElement:(Fallback) \"%ls\" node doesn't have a \"Name\" attribute.",
					*Utils::GetInfo(FieldElement));	
			}

			// Remember we where supposed to get into a MapElement.
			FieldElement.append_attribute("__HACK_SHOULD_BE__") = "MapElement";
		}

		Impl->Push(FieldElement);
		return; // fallback job done.
	}
	
	pugi::xml_node MapElementElement;
	
	if (!Impl->LastArrayElement.empty())
	{
		MapElementElement = Impl->LastArrayElement.next_sibling("MapElement");

		if (MapElementElement.empty())
		{
			UE_LOGF(LogXmlSerialization, Error,
				"FXmlArchiveInputFormatter::EnterMapElement: \"%ls\" node doesn't have a \"MapElement\" sibling.",
				*Utils::GetInfo(Impl->LastArrayElement)); 
		}
	}
	else
	{
		MapElementElement = Impl->XmlElement().child("MapElement");

		if (MapElementElement.empty())
		{
			UE_LOGF(LogXmlSerialization, Error,
				"FXmlArchiveInputFormatter::EnterMapElement: \"%ls\" node doesn't have a \"MapElement\" child.",
				*Utils::GetInfo(Impl->XmlElement())); 
		}		
	}
	
	if (!MapElementElement.empty())
	{
		const pugi::xml_attribute NameAttrib = MapElementElement.attribute("Name");
		OutName = NameAttrib.as_string();
		if (NameAttrib.empty())
		{
			UE_LOGF(LogXmlSerialization, Error,
				"FXmlArchiveInputFormatter::EnterMapElement: \"%ls\" node doesn't have a \"Name\" attribute.",
				*Utils::GetInfo(MapElementElement));	
		}
	}
	
	Impl->Push(MapElementElement);
}

void FXmlArchiveInputFormatter::LeaveMapElement()
{
	using namespace UE::XmlSerialization::Private;
	
	// Check if our current element should be something else (fallback code above).
	const pugi::xml_attribute HackAttrib = Impl->XmlElement().attribute("__HACK_SHOULD_BE__");
	if (!HackAttrib.empty())
	{
		// It is possible to be in a value node. If so pop it first.
		if (Utils::IsElementA(Impl->XmlElement(), TEXT("Value")))
		{
			Impl->Pop(TEXT("LeaveMapElement"), TEXT("Value"));
		}

		// Should be in a Field node. If so, keep track of the last element.
		Impl->LastArrayElement =  Utils::IsElementA(Impl->XmlElement(),TEXT("Field")) ? Impl->XmlElement() : pugi::xml_node();
		Impl->Pop(TEXT("LeaveMapElement"), TEXT("Field"));
	}
	else
	{
		Impl->LastArrayElement =  Utils::IsElementA(Impl->XmlElement(),TEXT("MapElement")) ? Impl->XmlElement() : pugi::xml_node();
		Impl->Pop(TEXT("LeaveMapElement"), TEXT("MapElement"));
	}
}

void FXmlArchiveInputFormatter::EnterAttributedValue()
{
	using namespace UE::XmlSerialization::Private;
	const pugi::xml_node ValueElement = Impl->XmlElement().child("AttributedValue");
	if (ValueElement.empty())
	{
		UE_LOGF(LogXmlSerialization, Error,
			"FXmlArchiveInputFormatter::EnterAttributedValue: Node \"%ls\" doesn't have an \"AttributedValue\" child.",
			*Utils::GetInfo(Impl->XmlElement())); 
	}
	Impl->Push(ValueElement);
}

void FXmlArchiveInputFormatter::EnterAttribute(FArchiveFieldName AttributeName)
{
	using namespace UE::XmlSerialization::Private;
	const pugi::xml_node AttributeElement = Impl->XmlElement().find_child_by_attribute("Attribute", "Name", STRING_CAST_UE_TO_PUGI(AttributeName.Name));
	if (AttributeElement.empty())
	{
		UE_LOGF(LogXmlSerialization, Error,
			"FXmlArchiveInputFormatter::EnterAttribute: Node \"%ls\" doesn't have an \"Attribute\" child of name \"%ls\".",
			*Utils::GetInfo(Impl->XmlElement()), AttributeName.Name);
	}
	
	Impl->Push(AttributeElement);
}

void FXmlArchiveInputFormatter::EnterAttributedValueValue()
{
	using namespace UE::XmlSerialization::Private;
	const pugi::xml_node ValueElement = Impl->XmlElement().child("AttributedValueValue");
	if (ValueElement.empty())
	{
		UE_LOGF(LogXmlSerialization, Error,
			"FXmlArchiveInputFormatter::EnterAttributedValueValue: Node \"%ls\" doesn't have an \"AttributedValueValue\" child.",
			*Utils::GetInfo(Impl->XmlElement())); 
	}
	Impl->Push(ValueElement);
}

bool FXmlArchiveInputFormatter::TryEnterAttributedValueValue()
{
	const pugi::xml_node ValueElement = Impl->XmlElement().child("AttributedValueValue");
	if (!ValueElement.empty())
	{
		Impl->Push(ValueElement);
		return true;
	}
	return false;
}

void FXmlArchiveInputFormatter::LeaveAttribute()
{
	using namespace UE::XmlSerialization::Private;
	// AttributedValueValue and Attribute are at the same level under AttributedValue,
	// We can pop them if requested.
	if (Utils::IsElementA(Impl->XmlElement(), TEXT("AttributedValueValue")))
	{
		Impl->Pop(TEXT("LeaveAttribute"), TEXT("AttributedValueValue"));
	}
	else
	{
		Impl->Pop(TEXT("LeaveAttribute"), TEXT("Attribute"));
	}
}

void FXmlArchiveInputFormatter::LeaveAttributedValue()
{
	using namespace UE::XmlSerialization::Private;
	// AttributedValueValue and Attribute seem to be used interchangeably, so we
	// need to pop out of them if popping out of AttributedValue.
	if (Utils::IsElementA(Impl->XmlElement(), TEXT("AttributedValueValue")))
	{
		Impl->Pop(TEXT("LeaveAttributedValue"), TEXT("AttributedValueValue"));
	}
	else if (Utils::IsElementA(Impl->XmlElement(), TEXT("Attribute")))
	{
		Impl->Pop(TEXT("LeaveAttributedValue"), TEXT("Attribute"));
	}
	
	Impl->Pop(TEXT("LeaveAttributedValue"), TEXT("AttributedValue"));
}

bool FXmlArchiveInputFormatter::TryEnterAttribute(FArchiveFieldName InAttributeName, bool bInEnterWhenSaving)
{
	const pugi::xml_node AttributeElement = Impl->XmlElement().find_child_by_attribute("Attribute", "Name", STRING_CAST_UE_TO_PUGI(InAttributeName.Name));
	if (!AttributeElement.empty())
	{
		Impl->Push(AttributeElement);
		return true;
	}
	return false;
}

void FXmlArchiveInputFormatter::Serialize(uint8& Value)
{
	const pugi::xml_attribute Attrib = Impl->GetValueAttrib();
	if (!Attrib.empty())
	{
		Value = Attrib.as_uint();
	}
}

void FXmlArchiveInputFormatter::Serialize(uint16& Value)
{
	const pugi::xml_attribute Attrib = Impl->GetValueAttrib();
	if (!Attrib.empty())
	{
		Value = Attrib.as_uint();
	}
}

void FXmlArchiveInputFormatter::Serialize(uint32& Value)
{
	const pugi::xml_attribute Attrib = Impl->GetValueAttrib();
	if (!Attrib.empty())
	{
		Value = Attrib.as_uint();
	}
}

void FXmlArchiveInputFormatter::Serialize(uint64& Value)
{
	const pugi::xml_attribute Attrib = Impl->GetValueAttrib();
	if (!Attrib.empty())
	{
		Value = Attrib.as_ullong();
	}
}

void FXmlArchiveInputFormatter::Serialize(int8& Value)
{
	const pugi::xml_attribute Attrib = Impl->GetValueAttrib();
	if (!Attrib.empty())
	{
		Value = Attrib.as_int();
	}
}

void FXmlArchiveInputFormatter::Serialize(int16& Value)
{
	const pugi::xml_attribute Attrib = Impl->GetValueAttrib();
	if (!Attrib.empty())
	{
		Value = Attrib.as_int();
	}
}

void FXmlArchiveInputFormatter::Serialize(int32& Value)
{
	const pugi::xml_attribute Attrib = Impl->GetValueAttrib();
	if (!Attrib.empty())
	{
		Value = Attrib.as_int();
	}
}

void FXmlArchiveInputFormatter::Serialize(int64& Value)
{
	const pugi::xml_attribute Attrib = Impl->GetValueAttrib();
	if (!Attrib.empty())
	{
		Value = Attrib.as_llong();
	}
}

void FXmlArchiveInputFormatter::Serialize(float& Value)
{
	const pugi::xml_attribute Attrib = Impl->GetValueAttrib();
	if (!Attrib.empty())
	{
		Value = Attrib.as_float();
	}
}
	
void FXmlArchiveInputFormatter::Serialize(double& Value)
{
	const pugi::xml_attribute Attrib = Impl->GetValueAttrib();
	if (!Attrib.empty())
	{
		Value = Attrib.as_double();
	}
}

void FXmlArchiveInputFormatter::Serialize(bool& Value)
{
	const pugi::xml_attribute Attrib = Impl->GetValueAttrib();
	if (!Attrib.empty())
	{
		Value = Attrib.as_bool();
	}
}

void FXmlArchiveInputFormatter::Serialize(UTF32CHAR& Value)
{
	const pugi::xml_attribute Attrib = Impl->GetValueAttrib();
	if (!Attrib.empty())
	{
		Value = Attrib.as_uint();
	}
}

void FXmlArchiveInputFormatter::Serialize(FString& Value)
{
	const pugi::xml_attribute Attrib = Impl->GetValueAttrib();
	if (!Attrib.empty())
	{
		Value = Attrib.as_string();	
		//Value.RemoveFromStart(TEXT("String:"));	// We removed that from serializer. Just a reminder here.
	}
}

void FXmlArchiveInputFormatter::Serialize(FName& Value)
{
	const pugi::xml_attribute Attrib = Impl->GetValueAttrib();
	if (!Attrib.empty())
	{
		const FString StringValue = Attrib.as_string();
		Value = FName(*StringValue);
	}
}

void FXmlArchiveInputFormatter::Serialize(UObject*& Value)
{
	// Special case for nested objects.
	if (IsNestedObject())
	{
		Value = LoadNestedObject();
	}
	else
	{
		const pugi::xml_attribute Attrib = Impl->GetValueAttrib();
		const FString StringValue = !Attrib.empty() ? Attrib.as_string() : FString();
		if (!StringValue.IsEmpty())
		{
			FPackageIndex ObjectIndex;
			LexFromString(ObjectIndex, *StringValue);
			Value = ResolveObject(ObjectIndex);
		}
		else
		{
			Value = nullptr;
		}
	}
}

void FXmlArchiveInputFormatter::Serialize(FText& Value)
{
	UE_LOGF(LogXmlSerialization, Error, "FXmlArchiveInputFormatter::Serialize: FText value is not supported by this formatter."); 

	// Unlike FJsonArchiveInputFormatter, my class is not friends with FText. :(
	
	// FStructuredArchive ChildArchive(*this);
	// FText::SerializeText(ChildArchive.Open(), Value);	---> is Private
	// ChildArchive.Close();
}

void FXmlArchiveInputFormatter::Serialize(FWeakObjectPtr& Value)
{
	UObject* Object = nullptr;
	Serialize(Object);
	Value = Object;
}

void FXmlArchiveInputFormatter::Serialize(FSoftObjectPtr& Value)
{
	// Special case for nested objects.
	if (IsNestedObject())
	{
		Value = LoadNestedObject();
	}
	else
	{
		FSoftObjectPath Path;
		Serialize(Path);
		Value = Path;
	}
}

void FXmlArchiveInputFormatter::Serialize(FSoftObjectPath& Value)
{
	const FString StringValue = Impl->GetValueAttrib().as_string();
	const auto& Prefix = TEXT("Object:");
	if (StringValue.StartsWith(Prefix))
	{
		Value.SetPath(*StringValue + UE_ARRAY_COUNT(Prefix) - 1);
	}
	else
	{
		Value.Reset();
	}
}

void FXmlArchiveInputFormatter::Serialize(FLazyObjectPtr& Value)
{
	const FString StringValue = Impl->GetValueAttrib().as_string();
	const auto& Prefix = TEXT("Lazy:");
	if (StringValue.StartsWith(Prefix))
	{
		FUniqueObjectGuid Guid;
		Guid.FromString(*StringValue + UE_ARRAY_COUNT(Prefix) - 1);
		Value = Guid;
	}
	else
	{
		Value.Reset();
	}
}

void FXmlArchiveInputFormatter::Serialize(FObjectPtr& Value)
{
	UObject* Object;
	Serialize(Object);
	Value = Object;
}

void FXmlArchiveInputFormatter::Serialize(TArray<uint8>& Data)
{
	// Single line string
	FString RawData = Impl->GetValueAttrib().as_string();
	if (!RawData.IsEmpty())
	{
		ensure(RawData.RemoveFromStart(TEXT("Base64:")));
		FBase64::Decode(RawData, Data);
	}
}

void FXmlArchiveInputFormatter::Serialize(void* Data, uint64 DataSize)
{
	TArray<uint8> Buffer;
	Serialize(Buffer);
	check(Buffer.Num() == DataSize);
	memcpy(Data, Buffer.GetData(), DataSize);
}

bool FXmlArchiveInputFormatter::IsNestedObject() const
{
	const pugi::xml_node RecordChild = Impl->XmlElement().child("Record");
	if (!RecordChild.empty() && RecordChild.attribute("Class").empty() == false)
	{
		return true;
	}
	return false;
}

UObject* FXmlArchiveInputFormatter::LoadNestedObject()
{
	using namespace UE::XmlSerialization::Private;
	const pugi::xml_node RecordChild = Impl->XmlElement().child("Record");
	if (!RecordChild.empty() && RecordChild.attribute("Class").empty() == false)
	{
		const FSoftObjectPath ClassPath(RecordChild.attribute("Class").as_string());
		const TSoftObjectPtr<UClass> ClassPtr(ClassPath);
		if (const UClass* const ObjectClass = ClassPtr.LoadSynchronous())
		{
			const EObjectFlags ObjectFlags = static_cast<EObjectFlags>(RecordChild.attribute("Flags").as_uint());
			const FString OriginalObjectName(RecordChild.attribute("Name").as_string());
			// Name clashing prevention: in order to prevent name clashing, we let the new object be named with
			// a unique name and rename the root field tag.
			UObject* const NestedObject = NewObject<UObject>(RootObject, ObjectClass, NAME_None, ObjectFlags);

			if (!NestedObject)
			{
				return nullptr;
			}

			{
				// Rename the root field tag's name attribute to match what the new object's name is.
				const pugi::xml_node FieldElement = RecordChild.find_child_by_attribute("Field", "Name", STRING_CAST_UE_TO_PUGI(*OriginalObjectName));
				if (!FieldElement.empty())
				{
					FieldElement.attribute("Name") = STRING_CAST_UE_TO_PUGI(*NestedObject->GetName());
				}
			}
			{
				FStructuredArchive ChildArchive(*this);
				FStructuredArchiveRecord ObjectRecord = ChildArchive.Open().EnterRecord();
				FormatterHelper::SerializeObject(ObjectRecord, NestedObject);
				ChildArchive.Close();
			}

			NestedObject->PostLoad();

			return NestedObject;
		}
	}
	return nullptr;
}

#endif
