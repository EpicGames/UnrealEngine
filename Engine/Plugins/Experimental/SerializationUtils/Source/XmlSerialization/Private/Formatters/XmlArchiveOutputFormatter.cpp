// Copyright Epic Games, Inc. All Rights Reserved.

#include "Formatters/XmlArchiveOutputFormatter.h"
#include "Serialization/MemoryWriter.h"
#include "Misc/Base64.h"
#include "Misc/SecureHash.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/ObjectPtr.h"

#include "Utils/XmlWriter.h"
#include "Utils/XmlUtils.h"
#include "XmlFormatterHelper.h"
#include "XmlSerializationModule.h"

#if WITH_TEXT_ARCHIVE_SUPPORT

class FXmlArchiveOutputFormatter::FImpl
{
public:
	FImpl()
	{
		pugi::xml_node decl = XmlDoc.prepend_child(pugi::node_declaration);
		decl.append_attribute("version") = "1.0";
		decl.append_attribute("encoding") = "utf-8";

		XmlElement = XmlDoc.append_child();
		XmlElement.set_name("ObjectFileDocument");
		XmlElement.append_attribute(("Version")) = "1.0";
	}

	// Note: for serialization, we are going to force popping.
	bool Pop(const TCHAR* FuncName, const TCHAR* TagName, bool bForce = true)
	{
		using namespace UE::XmlSerialization::Private;
		bool bPop = true;
		// Check if the tag name matches.
		if (!Utils::IsElementA(XmlElement, TagName))
		{
			UE_LOGF(LogXmlSerialization, Error,
				"FXmlArchiveOutputFormatter::%ls: Requested to leave \"%ls\", but it is \"%ls\".",
				FuncName, TagName, *Utils::GetInfo(XmlElement));
			bPop = false;
		}
		if (bPop || bForce)
		{
			XmlElement = XmlElement.parent();
		}
		return bPop;
	}

	template <typename ValueType>
	void SetValue(ValueType Value)
	{
		if (!XmlAttrib.empty())
		{
			XmlAttrib.set_value(Value);
		}
		else
		{
			// TODO: Investigate why this is not working.
			//XmlElement.set_value(*LexToString(Value));
			XmlElement.append_attribute(("Value")).set_value(Value);
		}
	}

	void SetValue(const TCHAR* Value)
	{
		if (!XmlAttrib.empty())
		{
			XmlAttrib.set_value(STRING_CAST_UE_TO_PUGI(Value));
		}
		else
		{
			// TODO: Investigate why this is not working.
			//XmlElement.set_value(Value);
			XmlElement.append_attribute(("Value")).set_value(STRING_CAST_UE_TO_PUGI(Value));
		}
	}

	
	pugi::xml_document XmlDoc;
	pugi::xml_node XmlElement;
	pugi::xml_attribute XmlAttrib;
	bool bDocumentSaveAttempted = false;
};

FXmlArchiveOutputFormatter::FXmlArchiveOutputFormatter(FArchive& InInner) 
	: Impl(new FImpl())
	, Inner(InInner)
{
	Inner.SetIsTextFormat(true);
}

FXmlArchiveOutputFormatter::~FXmlArchiveOutputFormatter()
{
	if (!Impl->bDocumentSaveAttempted)
	{
		if (!SaveDocumentToInnerArchive())
		{
			UE_LOGF(LogXmlSerialization, Error,
				"~FXmlArchiveOutputFormatter: Failed to save XML document to inner archive. Archive might be already destroyed, use SaveDocumentToInnerArchive() to ensure proper order.");
		}
	}
	delete Impl;
}

bool FXmlArchiveOutputFormatter::SaveDocumentToInnerArchive(EXmlSerializationEncoding InEncoding)
{
	using namespace UE::XmlSerialization::Private;
	// We let the user force saving again, but at least issue a warning about it.
	if (Impl->bDocumentSaveAttempted)
	{
		UE_LOGF(LogXmlSerialization, Warning,
			"FXmlArchiveOutputFormatter: Document has already been saved to archive. Prepare for unforeseen consequences.");
	}
	
	bool bSuccess = true;
	try
	{
		const pugi::xml_encoding encoding = Utils::ToPugiEncoding(InEncoding);
		Impl->bDocumentSaveAttempted = true;
		FXmlWriter XmlWriter(&Inner);
		Impl->XmlDoc.save(XmlWriter, "  ", pugi::format_default, encoding);
	}
	catch (...)
	{
		bSuccess = false;
	}
	return bSuccess;
}


FArchive& FXmlArchiveOutputFormatter::GetUnderlyingArchive()
{
	return Inner;
}

bool FXmlArchiveOutputFormatter::HasDocumentTree() const
{
	return true;
}

void FXmlArchiveOutputFormatter::EnterRecord()
{
	pugi::xml_node RecordElement = Impl->XmlElement.append_child(pugi::node_element);
	RecordElement.set_name("Record");
	Impl->XmlElement = RecordElement;	
}

void FXmlArchiveOutputFormatter::LeaveRecord()
{
	Impl->Pop(TEXT("LeaveRecord"), TEXT("Record"));
}

void FXmlArchiveOutputFormatter::EnterField(FArchiveFieldName Name)
{
	pugi::xml_node FieldElement = Impl->XmlElement.append_child(pugi::node_element);
	FieldElement.set_name("Field");
	FieldElement.append_attribute("Name") = STRING_CAST_UE_TO_PUGI(Name.Name);
	Impl->XmlElement = FieldElement;
}

void FXmlArchiveOutputFormatter::LeaveField()
{
	Impl->Pop(TEXT("LeaveField"),TEXT("Field"));
}

bool FXmlArchiveOutputFormatter::TryEnterField(FArchiveFieldName Name, bool bEnterWhenSaving)
{
	if (bEnterWhenSaving)
	{
		EnterField(Name);
	}
	return bEnterWhenSaving;
}

void FXmlArchiveOutputFormatter::EnterArray(int32& NumElements)
{
	pugi::xml_node ArrayElement = Impl->XmlElement.append_child(pugi::node_element);
	ArrayElement.set_name("Array");
	ArrayElement.append_attribute("Size").set_value(NumElements);
	Impl->XmlElement = ArrayElement;
}

void FXmlArchiveOutputFormatter::LeaveArray()
{
	Impl->Pop(TEXT("LeaveArray"),TEXT("Array"));
}

void FXmlArchiveOutputFormatter::EnterArrayElement()
{
	pugi::xml_node ArrayElementElement = Impl->XmlElement.append_child(pugi::node_element);
	ArrayElementElement.set_name("ArrayElement");
	Impl->XmlElement = ArrayElementElement;
}

void FXmlArchiveOutputFormatter::LeaveArrayElement()
{
	Impl->Pop(TEXT("LeaveArrayElement"), TEXT("ArrayElement"));
}

void FXmlArchiveOutputFormatter::EnterStream()
{
	pugi::xml_node StreamElement = Impl->XmlElement.append_child(pugi::node_element);
	StreamElement.set_name("Stream");
	Impl->XmlElement = StreamElement;
}

void FXmlArchiveOutputFormatter::LeaveStream()
{
	Impl->Pop(TEXT("LeaveStream"), TEXT("Stream"));
}

void FXmlArchiveOutputFormatter::EnterStreamElement()
{
	pugi::xml_node StreamElementElement = Impl->XmlElement.append_child(pugi::node_element);
	StreamElementElement.set_name("StreamElement");
	Impl->XmlElement = StreamElementElement;
}

void FXmlArchiveOutputFormatter::LeaveStreamElement()
{
	Impl->Pop(TEXT("LeaveStreamElement"),TEXT("StreamElement"));
}

void FXmlArchiveOutputFormatter::EnterMap(int32& NumElements)
{
	pugi::xml_node MapElement = Impl->XmlElement.append_child(pugi::node_element);
	MapElement.set_name("Map");
	MapElement.append_attribute("Size").set_value(NumElements);
	Impl->XmlElement = MapElement;
}

void FXmlArchiveOutputFormatter::LeaveMap()
{
	Impl->Pop(TEXT("LeaveMap"),TEXT("Map"));
}

void FXmlArchiveOutputFormatter::EnterMapElement(FString& Name)
{
	pugi::xml_node MapElementElement = Impl->XmlElement.append_child(pugi::node_element);
	MapElementElement.set_name("MapElement");
	MapElementElement.append_attribute("Name") = STRING_CAST_UE_TO_PUGI(*Name);
	Impl->XmlElement = MapElementElement;
}

void FXmlArchiveOutputFormatter::LeaveMapElement()
{
	Impl->Pop(TEXT("LeaveMapElement"),TEXT("MapElement"));
}

void FXmlArchiveOutputFormatter::EnterAttributedValue()
{
	pugi::xml_node ValueElement = Impl->XmlElement.append_child(pugi::node_element);
	ValueElement.set_name("AttributedValue");
	Impl->XmlElement = ValueElement;
}

void FXmlArchiveOutputFormatter::EnterAttribute(FArchiveFieldName AttributeName)
{
	pugi::xml_node AttributeElement = Impl->XmlElement.append_child(pugi::node_element);
	AttributeElement.set_name("Attribute");
	AttributeElement.append_attribute("Name") = STRING_CAST_UE_TO_PUGI(AttributeName.Name);
	Impl->XmlElement = AttributeElement;
	
	//checkf(FCString::Strcmp(AttributeName.Name, TEXT("Value")) != 0, TEXT("Attributes called 'Value' are reserved by the implementation"));
	//Impl->XmlAttrib = Impl->XmlElement.append_attribute(AttributeName.Name);
}

void FXmlArchiveOutputFormatter::LeaveAttribute()
{
	using namespace UE::XmlSerialization::Private;
	// It seems LeaveAttribute can be called after EnterAttributedValueValue.
	// So the sequence is:
	// EnterAttribute, EnterAttributedValueValue, LeaveAttribute.
	// 
	// Since AttributedValueValue and Attribute are at the same level under AttributedValue,
	// We can pop them interchangeably if requested.
	if (Utils::IsElementA(Impl->XmlElement, TEXT("AttributedValueValue")))
	{
		Impl->Pop(TEXT("LeaveAttribute"), TEXT("AttributedValueValue"));
	}
	else
	{
		Impl->Pop(TEXT("LeaveAttribute"), TEXT("Attribute"));
	}
	//Impl->XmlAttrib = pugi::xml_attribute();
}

void FXmlArchiveOutputFormatter::LeaveAttributedValue()
{
	if (FCStringAnsi::Strcmp(Impl->XmlElement.name(), "AttributedValueValue") == 0)
	{
		Impl->Pop(TEXT("LeaveAttributedValue"),TEXT("AttributedValueValue"));
	}	
	
	Impl->Pop(TEXT("LeaveAttributedValue"), TEXT("AttributedValue"));
}

void FXmlArchiveOutputFormatter::EnterAttributedValueValue()
{
	pugi::xml_node AttributedValueValueElement = Impl->XmlElement.append_child(pugi::node_element);
	AttributedValueValueElement.set_name("AttributedValueValue");
	Impl->XmlElement = AttributedValueValueElement;
}

bool FXmlArchiveOutputFormatter::TryEnterAttributedValueValue()
{
	return false;	// That's how it is in json as well.
}

bool FXmlArchiveOutputFormatter::TryEnterAttribute(FArchiveFieldName AttributeName, bool bEnterWhenSaving)
{
	if (bEnterWhenSaving)
	{
		EnterAttribute(AttributeName);
	}
	return bEnterWhenSaving;
}

void FXmlArchiveOutputFormatter::Serialize(uint8& Value)
{
	Impl->SetValue(Value);
}

void FXmlArchiveOutputFormatter::Serialize(uint16& Value)
{
	Impl->SetValue(Value);
}

void FXmlArchiveOutputFormatter::Serialize(uint32& Value)
{
	Impl->SetValue(Value);
}

void FXmlArchiveOutputFormatter::Serialize(uint64& Value)
{
	Impl->SetValue(Value);
}

void FXmlArchiveOutputFormatter::Serialize(int8& Value)
{
	Impl->SetValue(Value);
}

void FXmlArchiveOutputFormatter::Serialize(int16& Value)
{
	Impl->SetValue(Value);
}

void FXmlArchiveOutputFormatter::Serialize(int32& Value)
{
	Impl->SetValue(Value);
}

void FXmlArchiveOutputFormatter::Serialize(int64& Value)
{
	Impl->SetValue(Value);
}

void FXmlArchiveOutputFormatter::Serialize(float& Value)
{
	if ((float)(int)Value == Value)
	{
		Impl->SetValue((int)Value);
	}
	else
	{
		FString String = FString::Printf(TEXT("%.17g"), Value);
#if DO_GUARD_SLOW
		float RoundTripped;
		LexFromString(RoundTripped, *String);
		check(RoundTripped == Value);
#endif
		Impl->SetValue(*String);
	}
}

void FXmlArchiveOutputFormatter::Serialize(double& Value)
{
	if ((double)(int)Value == Value)
	{
		Impl->SetValue((int)Value);
	}
	else
	{
		FString String = FString::Printf(TEXT("%.17g"), Value);
#if DO_GUARD_SLOW
		double RoundTripped;
		LexFromString(RoundTripped, *String);
		check(RoundTripped == Value);
#endif
		Impl->SetValue(*String);
	}
}

void FXmlArchiveOutputFormatter::Serialize(bool& Value)
{
	Impl->SetValue(Value);
}

void FXmlArchiveOutputFormatter::Serialize(UTF32CHAR& Value)
{
	Impl->SetValue(*(uint32*)&Value);
}

void FXmlArchiveOutputFormatter::Serialize(FString& Value)
{
	Impl->SetValue(*Value);
	
	// Insert a "String:" prefix to prevent incorrect interpretation as another explicit type
	// if (Value.StartsWith(TEXT("Object:")) || Value.StartsWith(TEXT("String:")) || Value.StartsWith(TEXT("Base64:")))
	// {
	// 	SerializeStringInternal(FString::Printf(TEXT("String:%s"), *Value));
	// }
	// else
	// {
	// 	SerializeStringInternal(Value);
	// }
}

void FXmlArchiveOutputFormatter::Serialize(FName& Value)
{
	Impl->SetValue(*Value.ToString());
}

void FXmlArchiveOutputFormatter::Serialize(UObject*& Value)
{
	using namespace UE::XmlSerialization::Private;
	if (bSerializeObjectsInPlace)
	{
		if (Value != nullptr)
		{
			{
				Impl->XmlElement.append_child(pugi::node_comment).set_value("Object In Place");
			}
			
			FStructuredArchive ChildArchive(*this);
			FStructuredArchiveRecord ObjectRecord = ChildArchive.Open().EnterRecord();
			// For current record, add attribute for object type
			const TSoftObjectPtr<UClass> ObjectClass =  Value->GetClass();
			
			Impl->XmlElement.append_attribute("Class") = STRING_CAST_UE_TO_PUGI(*ObjectClass.ToSoftObjectPath().ToString());
			Impl->XmlElement.append_attribute("Name") = STRING_CAST_UE_TO_PUGI(*Value->GetName());
			Impl->XmlElement.append_attribute("Flags").set_value(static_cast<uint32>(Value->GetFlags()));			
			FormatterHelper::SerializeObject(ObjectRecord, Value);
			ChildArchive.Close();
		}
		else
		{
			Impl->SetValue(TEXT("null"));
		}
	}
	else
	{
		if (Value != nullptr && IsObjectAllowed(Value))
		{
			FPackageIndex ObjectIndex = ObjectIndicesMap->FindChecked(Value);
			Impl->SetValue(*LexToString(ObjectIndex));
		}
		else
		{
			Impl->SetValue(TEXT("null"));
		}
	}
}

void FXmlArchiveOutputFormatter::Serialize(FText& Value)
{
	// Unlike FJsonArchiveOutputFormatter, my class is not friends with FText. :(
	
	// FStructuredArchive ChildArchive(*this);
	// FText::SerializeText(ChildArchive.Open(), Value);	---> is Private
	// ChildArchive.Close();
}

void FXmlArchiveOutputFormatter::Serialize(FWeakObjectPtr& Value)
{
	UObject* Ptr = Value.IsValid() ? Value.Get() : nullptr;
	Serialize(Ptr);
}

void FXmlArchiveOutputFormatter::Serialize(FSoftObjectPtr& Value)
{
	if (bSerializeSoftObjectsInPlace)
	{
		UObject* Ptr = Value.IsValid() ? Value.Get() : nullptr;
		Serialize(Ptr);
	}
	else
	{
		FSoftObjectPath Path = Value.ToSoftObjectPath();
		Serialize(Path);
	}
}

void FXmlArchiveOutputFormatter::Serialize(FSoftObjectPath& Value)
{
	if (Value.IsValid())
	{
		Impl->SetValue(*FString::Printf(TEXT("Object:%s"), *Value.ToString()));
	}
	else
	{
		Impl->SetValue(TEXT("null"));
	}
}

void FXmlArchiveOutputFormatter::Serialize(FLazyObjectPtr& Value)
{
	if (Value.IsValid() && IsObjectAllowed(Value.Get()))
	{
		Impl->SetValue(*FString::Printf(TEXT("Lazy:%s"), *Value.GetUniqueID().ToString()));
	}
	else
	{
		Impl->SetValue(TEXT("null"));
	}
}

void FXmlArchiveOutputFormatter::Serialize(FObjectPtr& Value)
{
	UObject* Object = Value.Get();
	Serialize(Object);
}

void FXmlArchiveOutputFormatter::Serialize(TArray<uint8>& Data)
{
	Serialize(Data.GetData(), Data.Num());
}

void FXmlArchiveOutputFormatter::Serialize(void* Data, uint64 DataSize)
{
	// Encode the data on a single line. No need for hashing; intra-line merge conflicts are rare.
	Impl->SetValue(*FString::Printf(TEXT("Base64:%s"), *FBase64::Encode((const uint8*)Data, DataSize)));
}

bool FXmlArchiveOutputFormatter::IsObjectAllowed(UObject* InObject) const
{
	return ObjectIndicesMap && ObjectIndicesMap->Contains(InObject);
}
#endif
