// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmlFormatterHelper.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "Serialization/SerializedPropertyScope.h"

namespace UE::XmlSerialization::Private
{
	void FormatterHelper::SerializeObject(FStructuredArchiveRecord& InRootRecord
		, UObject* InObject
		, const TFunction<bool(FProperty*)>& ShouldSkipProperty)
	{
		if (!InObject)
		{
			return;
		}
	
		FStructuredArchiveRecord ObjectRecord = InRootRecord.EnterRecord(*InObject->GetName());
		FArchive& UnderlyingArchive = InRootRecord.GetUnderlyingArchive();
	
		for (FProperty* Property = InObject->GetClass()->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			if (!Property || Property->HasAnyPropertyFlags(CPF_Transient) || (ShouldSkipProperty && ShouldSkipProperty(Property)))
			{
				continue;
			}

			const FStructuredArchiveSlot PropertyField = ObjectRecord.EnterField(*Property->GetName());
		
			for (int32 Index = 0; Index < Property->ArrayDim; ++Index)
			{
				UObject* const Archetype = InObject->GetArchetype();
			
				void* const Data = Property->ContainerPtrToValuePtr<void>(InObject, Index);
				const void* const Defaults = Property->ContainerPtrToValuePtrForDefaults<void>(Archetype->GetClass()
					, Archetype
					, Index);
			
				//if (!Property->Identical(Target, Default, UnderlyingArchive.GetPortFlags()))
				{
					FSerializedPropertyScope SerializedProperty(UnderlyingArchive, Property);
					Property->SerializeItem(PropertyField, Data, Defaults);
				}
			}
		}
	}
}
