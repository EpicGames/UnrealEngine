// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"

class FProperty;
class FStructuredArchiveRecord;
class UObject;

namespace UE::XmlSerialization::Private::FormatterHelper
{
	void SerializeObject(FStructuredArchiveRecord& InRootRecord, UObject* InObject
		, const TFunction<bool(FProperty*)>& ShouldSkipProperty = TFunction<bool(FProperty*)>());
}