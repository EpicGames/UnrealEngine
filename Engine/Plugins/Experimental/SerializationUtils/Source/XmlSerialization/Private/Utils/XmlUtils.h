// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/Build.h"
#include "Misc/CString.h"
#include "XmlSerializationDefines.h"

#include "pugixml.hpp"

/**
 * pugixml can be compiled either in WChar mode (recommended) or Ansi mode.
 * It should define PUGIXML_WCHAR_MODE in pugiconfig.hpp if it has been compiled
 * in WChar mode.
 */
#ifdef PUGIXML_WCHAR_MODE
#	define STRING_CAST_UE_TO_PUGI(InString) InString
#	define STRING_CAST_PUGI_TO_UE(InString) InString
#else
#	define STRING_CAST_UE_TO_PUGI(InString) StringCast<ANSICHAR>(InString).Get() 
#	define STRING_CAST_PUGI_TO_UE(InString) StringCast<TCHAR>(InString).Get()
#endif

namespace UE::XmlSerialization::Private::Utils
{
	// Helper for pretty log
	inline FString GetInfo(const pugi::xml_node& XmlElement)
	{
		const pugi::xml_attribute NameAttrib = XmlElement.attribute("Name");
		if (!NameAttrib.empty())
		{
			return FString::Printf(TEXT("%s Name=\"%s\""), STRING_CAST_PUGI_TO_UE(XmlElement.name()), STRING_CAST_PUGI_TO_UE(NameAttrib.as_string()));
		}
		else
		{
			return FString(XmlElement.name());
		}
	}

	inline bool IsElementA(const pugi::xml_node& Element, const TCHAR* TagName)
	{
		return (FCStringAnsi::Strcmp(Element.name(), STRING_CAST_UE_TO_PUGI(TagName)) == 0) ? true : false;
	}

	inline bool IsElementShouldBe(const pugi::xml_node& Element, const TCHAR* TagName)
	{
		const pugi::xml_attribute HackAttrib = Element.attribute("__HACK_SHOULD_BE__");
		if (!HackAttrib.empty())
		{
			return (FCStringAnsi::Strcmp(HackAttrib.as_string(), STRING_CAST_UE_TO_PUGI(TagName)) == 0) ? true : false;
		}
		return false;
	}
	
	inline pugi::xml_encoding ToPugiEncoding(EXmlSerializationEncoding InEncoding)
	{
		switch (InEncoding)
		{
		case EXmlSerializationEncoding::Utf8:
			return pugi::encoding_utf8;
		case EXmlSerializationEncoding::Utf16_Le:
			return pugi::encoding_utf16_le;
		case EXmlSerializationEncoding::Utf16_Be:
			return pugi::encoding_utf16_be;
		case EXmlSerializationEncoding::Utf16:
			return pugi::encoding_utf16;
		case EXmlSerializationEncoding::Utf32_Le:
			return pugi::encoding_utf32_le;
		case EXmlSerializationEncoding::Utf32_Be:
			return pugi::encoding_utf32_be;
		case EXmlSerializationEncoding::Utf32:
			return pugi::encoding_utf32;
		case EXmlSerializationEncoding::WChar:
			return pugi::encoding_wchar;
		default:
			return pugi::encoding_utf8;
		}
	}
}