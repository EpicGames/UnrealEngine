// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/Archive.h"

#include "pugixml.hpp"

class FXmlWriter : public pugi::xml_writer
{
public:
	FXmlWriter(	FArchive* const InStream) : Stream(InStream) {}

	virtual void write(const void* data, size_t size) override
	{
		if (Stream != nullptr)
		{
			Stream->Serialize(const_cast<void*>(data), size);
		}
	}
private:
	FArchive* const Stream;
};
