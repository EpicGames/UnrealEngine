// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmlSerializationModule.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogXmlSerialization);

void FXmlSerializationModule::StartupModule()
{
}

void FXmlSerializationModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FXmlSerializationModule, XmlSerialization)