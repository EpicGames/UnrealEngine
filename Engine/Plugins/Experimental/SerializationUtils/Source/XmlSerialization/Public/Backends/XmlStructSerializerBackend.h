// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStructSerializerBackend.h"
#include "XmlSerializationDefines.h"

/**
 * Implements a writer for UStruct serialization using Xml.
 */
class XMLSERIALIZATION_API FXmlStructSerializerBackend
	: public IStructSerializerBackend
{
public:
	
	/**
	 * Creates and initializes a new instance with the given flags.
	 *
	 * @param InArchive The archive to serialize into.
	 * @param InFlags The flags that control the serialization behavior (typically EStructSerializerBackendFlags::Default).
	 */
	FXmlStructSerializerBackend( FArchive& InArchive, const EStructSerializerBackendFlags InFlags );

	virtual ~FXmlStructSerializerBackend() override;

public:
	
	/** Flush to archive. */
	void SaveDocument(EXmlSerializationEncoding InEncoding = EXmlSerializationEncoding::Utf8);

	//~ Begin IStructSerializerBackend
	virtual void BeginArray(const FStructSerializerState& InState) override;
	virtual void BeginStructure(const FStructSerializerState& InState) override;
	virtual void EndArray(const FStructSerializerState& InState) override;
	virtual void EndStructure(const FStructSerializerState& InState) override;
	virtual void WriteComment(const FString& InComment) override;
	virtual void WriteProperty(const FStructSerializerState& InState, int32 InArrayIndex) override;
	//~ End IStructSerializerBackend
	
private:
	// We keep implementation private so we don't expose the xml lib used.
	class FImpl;
	FImpl* Impl; 
	
	/** Flags controlling the serialization behavior. */
	EStructSerializerBackendFlags Flags;
};
