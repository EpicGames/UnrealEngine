// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/StructuredArchiveFormatter.h"
#include "UObject/ObjectResource.h"

#if WITH_TEXT_ARCHIVE_SUPPORT

class XMLSERIALIZATION_API FXmlArchiveInputFormatter final : public FStructuredArchiveFormatter
{
public:
	/**
	 * Parse the given archive into an xml DOM for deserializing objects.
	 *
	 * @param InInner Inner archive to read from. The whole archive is read in the constructor to parse into xml DOM.
	 * @param InRootObject Specifies the root object to be the owner for nested objects. Can't be null.
	 * @param InResolveObject Resolves reference for the UObject properties. This is not used if objects where serialized nested. 
	 */
	FXmlArchiveInputFormatter(FArchive& InInner, UObject* InRootObject, const TFunction<UObject* (const FPackageIndex)>& InResolveObject = nullptr);
	FXmlArchiveInputFormatter(const FXmlArchiveInputFormatter& InOther);

	virtual ~FXmlArchiveInputFormatter() override;

	bool IsParseResultStatusOk() const;

	virtual FArchive& GetUnderlyingArchive() override;
	virtual FStructuredArchiveFormatter* CreateSubtreeReader() override;

	virtual bool HasDocumentTree() const override;

	virtual void EnterRecord() override;
	virtual void LeaveRecord() override;
	virtual void EnterField(FArchiveFieldName InName) override;
	virtual void LeaveField() override;
	virtual bool TryEnterField(FArchiveFieldName InName, bool bInEnterWhenSaving) override;

	virtual void EnterArray(int32& OutNumElements) override;
	virtual void LeaveArray() override;
	virtual void EnterArrayElement() override;
	virtual void LeaveArrayElement() override;

	virtual void EnterStream() override;
	virtual void LeaveStream() override;
	virtual void EnterStreamElement() override;
	virtual void LeaveStreamElement() override;

	virtual void EnterMap(int32& OutNumElements) override;
	virtual void LeaveMap() override;
	virtual void EnterMapElement(FString& OutName) override;
	virtual void LeaveMapElement() override;

	virtual void EnterAttributedValue() override;
	virtual void EnterAttribute(FArchiveFieldName InAttributeName) override;
	virtual void EnterAttributedValueValue() override;
	virtual void LeaveAttribute() override;
	virtual void LeaveAttributedValue() override;
	virtual bool TryEnterAttribute(FArchiveFieldName InAttributeName, bool bInEnterWhenSavin) override;
	virtual bool TryEnterAttributedValueValue() override;

	virtual void Serialize(uint8& Value) override;
	virtual void Serialize(uint16& Value) override;
	virtual void Serialize(uint32& Value) override;
	virtual void Serialize(uint64& Value) override;
	virtual void Serialize(int8& Value) override;
	virtual void Serialize(int16& Value) override;
	virtual void Serialize(int32& Value) override;
	virtual void Serialize(int64& Value) override;
	virtual void Serialize(float& Value) override;
	virtual void Serialize(double& Value) override;
	virtual void Serialize(bool& Value) override;
	virtual void Serialize(UTF32CHAR& Value) override;
	virtual void Serialize(FString& Value) override;
	virtual void Serialize(FName& Value) override;
	virtual void Serialize(UObject*& Value) override;
	virtual void Serialize(FText& Value) override;
	virtual void Serialize(FWeakObjectPtr& Value) override;
	virtual void Serialize(FSoftObjectPtr& Value) override;
	virtual void Serialize(FSoftObjectPath& Value) override;
	virtual void Serialize(FLazyObjectPtr& Value) override;
	virtual void Serialize(FObjectPtr& Value) override;
	virtual void Serialize(TArray<uint8>& Value) override;
	virtual void Serialize(void* Data, uint64 DataSize) override;

private:
	bool IsNestedObject() const;
	UObject* LoadNestedObject();

private:
	class FImpl;
	FImpl* Impl;
	
	FArchive& Inner;
	TFunction<UObject* (const FPackageIndex)> ResolveObject;
	UObject* RootObject;
};

#endif
