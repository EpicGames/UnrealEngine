// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** These flags determine the encoding of input data for XML document. */
enum class EXmlSerializationEncoding
{
	/** UTF8 encoding */
	Utf8,
	/** Little-endian UTF16 */
	Utf16_Le,
	/** Big-endian UTF16 */
	Utf16_Be,
	/** UTF16 with native endianness */
	Utf16,
	/** Little-endian UTF32 */
	Utf32_Le,
	/** Big-endian UTF32 */
	Utf32_Be,
	/** UTF32 with native endianness */
	Utf32,
	/** The same encoding wchar_t has (either UTF16 or UTF32) */
	WChar,
};